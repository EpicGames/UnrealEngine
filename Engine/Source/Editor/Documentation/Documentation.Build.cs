// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class Documentation : ModuleRules
	{
		public Documentation(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"MainFrame"
					// ... add other public dependencies that you statically link with here ...
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					// ... add private dependencies that you statically link with here ...
                    "CoreUObject",
                    "Engine",
                    "InputCore",
                    "Slate",
					"SlateCore",
					"DeveloperSettings",
					"Projects",
					"EditorFramework",
                    "UnrealEd",
					"Analytics",
					"SourceCodeAccess",
					"SourceControl",
                    "ContentBrowser",
					"DesktopPlatform",
					"ToolWidgets",
				}
			);

			CircularlyReferencedDependentModules.AddRange(
				[
					"ContentBrowser",
					"SourceControl",
				]);

			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
                    "MessageLog"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"MessageLog"
				}
			);
		}
	}
}
