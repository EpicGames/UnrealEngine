// Copyright Epic Games, Inc. All Rights Reserved.

#include "Documentation.h"

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Dialogs/Dialogs.h"
#include "DocumentationLink.h"
#include "DocumentationPage.h"
#include "DocumentationSettings.h"
#include "EngineAnalytics.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "IAnalyticsProviderET.h"
#include "Interfaces/IMainFrameModule.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/CommandLine.h"
#include "Misc/PackageName.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/ReverseIterate.h"
#include "Modules/ModuleManager.h"
#include "SDocumentationAnchor.h"
#include "SDocumentationToolTip.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "UDNParser.h"
#include "UnrealEdMisc.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SToolTip.h"
#include "Misc/App.h"

class IDocumentationPage;
class SVerticalBox;
class SWidget;

#define LOCTEXT_NAMESPACE "DocumentationActor"

DEFINE_LOG_CATEGORY(LogDocumentation);

TSharedRef< IDocumentation > FDocumentation::Create() 
{
	return MakeShareable( new FDocumentation() );
}

FDocumentation::FDocumentation() 
{
	if (FApp::CanEverRender() && !IsRunningCommandlet())
	{
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		MainFrameModule.OnMainFrameSDKNotInstalled().AddRaw(this, &FDocumentation::HandleSDKNotInstalled);

		if (const UDocumentationSettings* DocSettings = GetDefault<UDocumentationSettings>())
		{
			for (const FDocumentationBaseUrl& BaseUrl : DocSettings->DocumentationBaseUrls)
			{
				if (!RegisterBaseUrl(BaseUrl.Id, BaseUrl.Url))
				{
					UE_LOGF(LogDocumentation, Warning, "Could not register documentation base URL: %ls", *BaseUrl.Id);
				}
			}
		}

		AddSourcePath(FPaths::Combine(FPaths::ProjectDir(), TEXT("Documentation"), TEXT("Source")));
		for (TSharedRef<IPlugin> Plugin : IPluginManager::Get().GetEnabledPlugins())
		{
			AddSourcePath(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Documentation"), TEXT("Source")));
		}
		AddSourcePath(FPaths::Combine(FPaths::EngineDir(), TEXT("Documentation"), TEXT("Source")));
	}

	RegisterConfigRedirects();
}

FDocumentation::~FDocumentation() 
{

}

bool FDocumentation::OpenHome(FDocumentationSourceInfo Source, const FString& BaseUrlId) const
{
	return Open(TEXT("%ROOT%"), Source, BaseUrlId);
}

bool FDocumentation::OpenHome(const FCultureRef& Culture, FDocumentationSourceInfo Source, const FString& BaseUrlId) const
{
	return Open(TEXT("%ROOT%"), Culture, Source, BaseUrlId);
}

bool FDocumentation::OpenAPIHome(FDocumentationSourceInfo Source) const
{
	FString Url;
	FUnrealEdMisc::Get().GetURL(TEXT("APIDocsURL"), Url, true);

	if (!Url.IsEmpty())
	{
		FUnrealEdMisc::Get().ReplaceDocumentationURLWildcards(Url, FInternationalization::Get().GetCurrentCulture());
		FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);

		return true;
	}
	return false;
}

bool FDocumentation::Open(const FString& Link, FDocumentationSourceInfo Source, const FString& BaseUrlId) const
{
	return OpenUrl(Link, FInternationalization::Get().GetCurrentCulture(), Source, BaseUrlId);
}

bool FDocumentation::Open(const FString& Link, const FCultureRef& Culture, FDocumentationSourceInfo Source, const FString& BaseUrlId) const
{
	return OpenUrl(Link, Culture, Source, BaseUrlId);
}

TSharedRef< SWidget > FDocumentation::CreateAnchor( const TAttribute<FString>& Link, const FString& PreviewLink, const FString& PreviewExcerptName, const TAttribute<FString>& BaseUrlId) const
{
	return SNew( SDocumentationAnchor )
		.Link(Link)
		.PreviewLink(PreviewLink)
		.PreviewExcerptName(PreviewExcerptName)
		.BaseUrlId(BaseUrlId);
}

TSharedRef< IDocumentationPage > FDocumentation::GetPage( const FString& Link, const TSharedPtr< FParserConfiguration >& Config, const FDocumentationStyle& Style )
{
	TSharedPtr< IDocumentationPage > Page;
	const TWeakPtr< IDocumentationPage >* ExistingPagePtr = LoadedPages.Find( Link );

	if ( ExistingPagePtr != NULL )
	{
		const TSharedPtr< IDocumentationPage > ExistingPage = ExistingPagePtr->Pin();
		if ( ExistingPage.IsValid() )
		{
			Page = ExistingPage;
		}
	}

	if ( !Page.IsValid() )
	{
		Page = FDocumentationPage::Create( Link, FUDNParser::Create( Config, Style ) );
		LoadedPages.Add( Link, TWeakPtr< IDocumentationPage >( Page ) );
	}

	return Page.ToSharedRef();
}

bool FDocumentation::PageExists(const FString& Link) const
{
	const TWeakPtr< IDocumentationPage >* ExistingPagePtr = LoadedPages.Find(Link);
	if (ExistingPagePtr != NULL)
	{
		return true;
	}

	for (const FString& SourcePath : SourcePaths)
	{
		FString LinkPath = FDocumentationLink::ToSourcePath(Link, SourcePath);
		if (FPaths::FileExists(LinkPath))
		{
			return true;
		}
	}
	return false;
}

bool FDocumentation::PageExists(const FString& Link, const FCultureRef& Culture) const
{
	const TWeakPtr< IDocumentationPage >* ExistingPagePtr = LoadedPages.Find(Link);
	if (ExistingPagePtr != NULL)
	{
		return true;
	}

	for (const FString& SourcePath : SourcePaths)
	{
		FString LinkPath = FDocumentationLink::ToSourcePath(Link, Culture, SourcePath);
		if (FPaths::FileExists(LinkPath))
		{
			return true;
		}
	}
	return false;
}

const TArray <FString>& FDocumentation::GetSourcePaths() const
{
	return SourcePaths;
}

TSharedRef< class SToolTip > FDocumentation::CreateToolTip(const TAttribute<FText>& Text, const TSharedPtr<SWidget>& OverrideContent, const FString& Link, const FString& ExcerptName) const
{
	return CreateToolTip(Text, OverrideContent, Link, ExcerptName, FText());
}

TSharedRef< class SToolTip > FDocumentation::CreateToolTip(const TAttribute<FText>& Text, const TSharedPtr<SWidget>& OverrideContent, const FString& Link, const FString& ExcerptName, const TAttribute<FText>& Shortcut) const
{
	TSharedPtr< SDocumentationToolTip > DocToolTip;

	if ( !Text.IsBound() && Text.Get().IsEmpty() )
	{
		return SNew( SToolTip );
	}

	if ( OverrideContent.IsValid() )
	{
		SAssignNew( DocToolTip, SDocumentationToolTip )
		.DocumentationLink( Link )
		.ExcerptName( ExcerptName )
		.Shortcut(Shortcut)
		[
			OverrideContent.ToSharedRef()
		];
	}
	else
	{
		SAssignNew(DocToolTip, SDocumentationToolTip)
		.Text(Text)
		.DocumentationLink(Link)
		.ExcerptName(ExcerptName)
		.Shortcut(Shortcut);
	}
	
	return SNew( SToolTip )
		.IsInteractive( DocToolTip.ToSharedRef(), &SDocumentationToolTip::IsInteractive )

		// Emulate text-only tool-tip styling that SToolTip uses when no custom content is supplied.  We want documentation tool-tips to 
		// be styled just like text-only tool-tips
		.BorderImage( FCoreStyle::Get().GetBrush("ToolTip.BrightBackground") )
		// The documentation tooltip has padding built in, so that it can style its own background within the tooltip borders.
		.TextMargin(FMargin(0.0f))
		[
			DocToolTip.ToSharedRef()
		];
}

TSharedRef< class SToolTip > FDocumentation::CreateToolTip(const TAttribute<FText>& Text, const TSharedRef<SWidget>& OverrideContent, const TSharedPtr<SVerticalBox>& DocVerticalBox, const FString& Link, const FString& ExcerptName) const
{
	TSharedRef<SDocumentationToolTip> DocToolTip =
		SNew(SDocumentationToolTip)
		.Text(Text)
		.DocumentationLink(Link)
		.ExcerptName(ExcerptName)
		.AddDocumentation(false)
		.DocumentationMargin(7)
		[
			OverrideContent
		];

	if (DocVerticalBox.IsValid())
	{
		DocToolTip->AddDocumentation(DocVerticalBox);
	}

	return SNew(SToolTip)
		.IsInteractive(DocToolTip, &SDocumentationToolTip::IsInteractive)

		// Emulate text-only tool-tip styling that SToolTip uses when no custom content is supplied.  We want documentation tool-tips to 
		// be styled just like text-only tool-tips
		.BorderImage( FCoreStyle::Get().GetBrush("ToolTip.BrightBackground") )
		// The documentation tooltip has padding built in, so that it can style its own background within the tooltip borders.
		.TextMargin(FMargin(0.0f))
		[
			DocToolTip
		];
}

void FDocumentation::HandleSDKNotInstalled(const FString& PlatformName, const FString& InDocumentationPage)
{
	if (FPackageName::IsValidLongPackageName(InDocumentationPage, true))
	{
		return;
	}
	IDocumentation::Get()->Open(InDocumentationPage);
}

bool FDocumentation::RegisterBaseUrl(const FString& Id, const FString& Url)
{
	if (!Id.IsEmpty() && !Url.IsEmpty())
	{
		if (!RegisteredBaseUrls.Contains(Id))
		{
			RegisteredBaseUrls.Add(Id, Url);
			return true;
		}
		UE_LOGF(LogDocumentation, Warning, "Could not register documentation base URL with ID: %ls. This ID is already in use.", *Id);
		return false;
	}
	return false;
}

FString FDocumentation::GetBaseUrl(const FString& Id) const
{
	if (!Id.IsEmpty())
	{
		const FString* BaseUrl = RegisteredBaseUrls.Find(Id);
		if (BaseUrl != NULL && !BaseUrl->IsEmpty())
		{
			return *BaseUrl;
		}
		UE_LOGF(LogDocumentation, Warning, "Could not resolve base URL with ID: %ls. It may not have been registered.", *Id);
	}

	FString DefaultUrl;
	FUnrealEdMisc::Get().GetURL(TEXT("DocumentationURL"), DefaultUrl, true);
	return DefaultUrl;
}

bool FDocumentation::AddSourcePath(const FString& Path)
{
	if (!Path.IsEmpty() && FPaths::DirectoryExists(Path))
	{
		SourcePaths.Add(Path);
		return true;
	}
	return false;
}

bool FDocumentation::RegisterRedirect(const FName& Owner, const FDocumentationRedirect& Redirect)
{
	return RedirectRegistry.Register(Owner, Redirect);
}

void FDocumentation::UnregisterRedirects(const FName& Owner)
{
	RedirectRegistry.UnregisterAll(Owner);
}

bool FDocumentation::OpenUrl(const FString& Link, const FCultureRef& Culture, FDocumentationSourceInfo Source, const FString& BaseUrlId) const
{
	// Original url
	const FDocumentationUrl OriginalUrl(Link, BaseUrlId);

	// Target url (redirected if applicable)
	const FDocumentationUrl TargetUrl = RedirectUrl(OriginalUrl);

	// Warn the user if they are opening a URL
	if (TargetUrl.Link.StartsWith(TEXT("http")) || TargetUrl.Link.StartsWith(TEXT("https")))
	{
		FText Message = LOCTEXT("OpeningURLMessage", "You are about to open an external URL. This will open your web browser. Do you want to proceed?");
		FText URLDialog = LOCTEXT("OpeningURLTitle", "Open external link");

		FSuppressableWarningDialog::FSetupInfo Info(Message, URLDialog, "SuppressOpenURLWarning");
		Info.ConfirmText = LOCTEXT("OpenURL_yes", "Yes");
		Info.CancelText = LOCTEXT("OpenURL_no", "No");
		FSuppressableWarningDialog OpenURLWarning(Info);
		if (OpenURLWarning.ShowModal() == FSuppressableWarningDialog::Cancel)
		{
			return false;
		}
		else
		{
			FPlatformProcess::LaunchURL(*TargetUrl.Link, nullptr, nullptr);
			return true;
		}
	}

	FString DocumentationUrl;

	if (!FParse::Param(FCommandLine::Get(), TEXT("testdocs")))
	{
		// Prioritize on-disk versions of the requested link over web versions
		const FString DiskPath = FDocumentationLink::ToFilePath(TargetUrl.Link, Culture);
		if (FPaths::FileExists(DiskPath))
		{
			DocumentationUrl = FDocumentationLink::ToFileUrl(TargetUrl.Link, Culture, Source);
		}
		else if (const FCulturePtr FallbackCulture = FInternationalization::Get().GetCulture(TEXT("en")))
		{
			const FString FallbackDiskPath = FDocumentationLink::ToFilePath(TargetUrl.Link, FallbackCulture.ToSharedRef());
			if (FPaths::FileExists(FallbackDiskPath))
			{
				DocumentationUrl = FDocumentationLink::ToFileUrl(TargetUrl.Link, FallbackCulture.ToSharedRef(), Source);
			}
		}
	}

	if (DocumentationUrl.IsEmpty())
	{
		DocumentationUrl = FDocumentationLink::ToUrl(TargetUrl.Link, Culture, Source, TargetUrl.BaseUrlId);
	}

	if (!DocumentationUrl.IsEmpty())
	{
		FPlatformProcess::LaunchURL(*DocumentationUrl, nullptr, nullptr);

		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Documentation"), TEXT("OpenedPage"), TargetUrl.Link);
		}

		return true;
	}
	else
	{
		return false;
	}
}

FDocumentationUrl FDocumentation::RedirectUrl(const FDocumentationUrl& OriginalUrl) const
{
	FDocumentationRedirect Redirect;
	if (!RedirectRegistry.GetRedirect(OriginalUrl.Link, Redirect))
	{
		return OriginalUrl;
	}
	else
	{
		UE_LOGF(LogDocumentation, Log, "Documentation link \"%ls\" was redirected to \"%ls\"", *OriginalUrl.ToString(), *Redirect.ToUrl.ToString());
		return Redirect.ToUrl;
	}
}

void FDocumentation::RegisterConfigRedirects()
{
	// Unregister all global redirects so we start with a clean slate every time we parse the config redirects
	UnregisterRedirects(NAME_None);

	const UDocumentationSettings* DocumentationSettings = GetDefault<UDocumentationSettings>();
	if (!DocumentationSettings)
	{
		return;
	}

	// Iterate over redirects in reverse so the redirects deepest in the config hierarchy are registered last (higher priority)
	for (const FDocumentationRedirect& ConfigRedirect : ReverseIterate(DocumentationSettings->DocumentationRedirects))
	{
		RegisterRedirect(NAME_None, ConfigRedirect);
	}
}

#undef LOCTEXT_NAMESPACE
