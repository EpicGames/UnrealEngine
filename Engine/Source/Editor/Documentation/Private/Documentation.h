// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DocumentationRedirectRegistry.h"
#include "IDocumentation.h"
#include "Internationalization/CulturePointer.h"
#include "Templates/SharedPointer.h"

class FText;
class IDocumentationPage;
class SVerticalBox;
class SWidget;
template< typename ObjectType > class TAttribute;

DECLARE_LOG_CATEGORY_EXTERN(LogDocumentation, Log, All);

class FDocumentation : public IDocumentation
{
public:

	static TSharedRef< IDocumentation > Create();

public:

	virtual ~FDocumentation();

	virtual bool OpenHome(FDocumentationSourceInfo Source = FDocumentationSourceInfo(), const FString& BaseUrlId = FString()) const override;

	virtual bool OpenHome(const FCultureRef& Culture, FDocumentationSourceInfo Source = FDocumentationSourceInfo(), const FString& BaseUrlId = FString()) const override;

	virtual bool OpenAPIHome(FDocumentationSourceInfo Source = FDocumentationSourceInfo()) const override;

	virtual bool Open( const FString& Link, FDocumentationSourceInfo Source = FDocumentationSourceInfo(), const FString& BaseUrlId = FString()) const override;

	virtual bool Open( const FString& Link, const FCultureRef& Culture, FDocumentationSourceInfo Source = FDocumentationSourceInfo(), const FString& BaseUrlId = FString()) const override;

	virtual TSharedRef< SWidget > CreateAnchor( const TAttribute<FString>& Link, const FString& PreviewLink = FString(), const FString& PreviewExcerptName = FString(), const TAttribute<FString>& BaseUrlId = FString()) const override;

	virtual TSharedRef< IDocumentationPage > GetPage( const FString& Link, const TSharedPtr< FParserConfiguration >& Config, const FDocumentationStyle& Style = FDocumentationStyle() ) override;

	virtual bool PageExists(const FString& Link) const override;

	virtual bool PageExists(const FString& Link, const FCultureRef& Culture) const override;

	virtual const TArray < FString >& GetSourcePaths() const override;

	virtual TSharedRef< class SToolTip > CreateToolTip( const TAttribute<FText>& Text, const TSharedPtr<SWidget>& OverrideContent, const FString& Link, const FString& ExcerptName ) const override;
	
	virtual TSharedRef< class SToolTip > CreateToolTip(const TAttribute<FText>& Text, const TSharedPtr<SWidget>& OverrideContent, const FString& Link, const FString& ExcerptName, const TAttribute<FText>& Shortcut) const override;

	virtual TSharedRef< class SToolTip > CreateToolTip(const TAttribute<FText>& Text, const TSharedRef<SWidget>& OverrideContent, const TSharedPtr<SVerticalBox>& DocVerticalBox, const FString& Link, const FString& ExcerptName) const override;

	virtual bool RegisterBaseUrl(const FString& Id, const FString& Url) override;

	virtual FString GetBaseUrl(const FString& Id) const override;

	virtual bool RegisterRedirect(const FName& Owner, const FDocumentationRedirect& Redirect) override;

	virtual void UnregisterRedirects(const FName& Owner) override;

private:

	FDocumentation();
	
	/**
	 * Builds and opens a documentation url given individual components/parameters
	 * @param Link Identifier for the page (could be a full url or individual page id)
	 * @param Culture Desired locale for resulting documentation url
	 * @param Source Metadata about the page request used by analytics
	 * @return Was the documentation url successfully opened?
	 */
	bool OpenUrl(const FString& Link, const FCultureRef& Culture, FDocumentationSourceInfo Source, const FString& BaseUrlId) const;

	/**
	 * Redirects a documentation url
	 * @param OriginalUrl Original url
	 * @return Redirected documentation url
	 */
	FDocumentationUrl RedirectUrl(const FDocumentationUrl& OriginalUrl) const;

	/** Registers all redirects from editor config file */
	void RegisterConfigRedirects();

private:

	TMap< FString, TWeakPtr< IDocumentationPage > > LoadedPages;

	TMap< const FString, const FString > RegisteredBaseUrls;

	FDocumentationRedirectRegistry RedirectRegistry;

	TArray < FString > SourcePaths;

	void HandleSDKNotInstalled(const FString& PlatformName, const FString& InDocumentationPage);

	bool AddSourcePath(const FString& Path);
};
