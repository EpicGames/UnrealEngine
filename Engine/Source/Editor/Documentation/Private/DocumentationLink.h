// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IDocumentation.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Internationalization/Culture.h"
#include "UnrealEdMisc.h"

class FDocumentationLink
{
public: 

	static FString GetUrlRoot(const FString& BaseUrlId = FString())
	{
		return IDocumentation::Get()->GetBaseUrl(BaseUrlId);
	}

	static FString GetHomeUrl()
	{
		return GetHomeUrl(FInternationalization::Get().GetCurrentCulture());
	}

	static FString GetHomeUrl(const FCultureRef& Culture)
	{
		FString Url;
		FUnrealEdMisc::Get().GetURL( TEXT("DocumentationURL"), Url, true );
		FUnrealEdMisc::Get().ReplaceDocumentationURLWildcards(Url, Culture);

		return Url;
	}

	static FString ToUrl(const FString& Link, FDocumentationSourceInfo const& Source, const FString& BaseUrlId = FString())
	{
		return ToUrl(Link, FInternationalization::Get().GetCurrentCulture(), Source, BaseUrlId);
	}

	static FString ToUrl(const FString& Link, const FCultureRef& Culture, FDocumentationSourceInfo const& Source, const FString& BaseUrlId = FString())
	{
		// Get the base URL for this doc request, if any.
		// It may have any or all of the following pieces in it.
		FString UrlRoot = GetUrlRoot(BaseUrlId);
		FString RootPath;
		FString QueryString;
		FString Anchor;
		SplitLink(UrlRoot, RootPath, QueryString, Anchor);

		// Break up the incoming documentation link into pieces.
		// Depending on where it comes from, the link may be just a page ID, or it may have any or all
		// of the following pieces in it. If there's a query string, SplitLink will merge it with the existing query
		// string found in the base URL (if any). If there's an anchor, SplitLink will use it /instead/ of the
		// one in the BaseUrl (if any).
		FString LinkPath;
		SplitLink(Link, LinkPath, QueryString, Anchor);
		const FString TrimmedPartialPath = LinkPath.TrimChar('/');

		// Add query parameters for the source, if needed.
		AddSourceInfoToQueryString(QueryString, Source);

		FString AssembledUrl = RootPath + QueryString + Anchor;
		FUnrealEdMisc::Get().ReplaceDocumentationURLWildcards(AssembledUrl, Culture, TrimmedPartialPath);
		return AssembledUrl;
	}

	static FString ToFilePath( const FString& Link )
	{
		FInternationalization& I18N = FInternationalization::Get();

		FString FilePath = ToFilePath(Link, I18N.GetCurrentCulture());

		if (!FPaths::FileExists(FilePath))
		{
			const FCulturePtr FallbackCulture = I18N.GetCulture(TEXT("en"));
			if (FallbackCulture.IsValid())
			{
				const FString FallbackFilePath = ToFilePath(Link, FallbackCulture.ToSharedRef());
				if (FPaths::FileExists(FallbackFilePath))
				{
					FilePath = FallbackFilePath;
				}
			}
		}

		return FilePath;
	}

	static FString ToFilePath(const FString& Link, const FCultureRef& Culture)
	{
		FString Path;
		FString Anchor;
		FString QueryString;
		SplitLink(Link, Path, QueryString, Anchor);

		static FString Version = FString::FromInt(FEngineVersion::Current().GetMajor()) + TEXT(".") + FString::FromInt(FEngineVersion::Current().GetMinor());
		const FString PartialPath = FString::Printf(TEXT("%s/%s%s/index.html"), *Version, *(Culture->GetName()), *Path);
		return FString::Printf(TEXT("%sDocumentation/HTML/%s"), *FPaths::ConvertRelativePathToFull(FPaths::EngineDir()), *PartialPath);
	}

	static FString ToFileUrl(const FString& Link, FDocumentationSourceInfo const& SourceInfo)
	{
		FInternationalization& I18N = FInternationalization::Get();

		FCultureRef Culture = I18N.GetCurrentCulture();
		FString FilePath = ToFilePath(Link, Culture);

		if (!FPaths::FileExists(FilePath))
		{
			const FCulturePtr FallbackCulture = I18N.GetCulture(TEXT("en"));
			if (FallbackCulture.IsValid())
			{
				const FString FallbackFilePath = ToFilePath(Link, FallbackCulture.ToSharedRef());
				if (FPaths::FileExists(FallbackFilePath))
				{
					Culture = FallbackCulture.ToSharedRef();
				}
			}
		}

		return ToFileUrl(Link, Culture, SourceInfo);
	}

	static FString ToFileUrl(const FString& Link, const FCultureRef& Culture, FDocumentationSourceInfo const& SourceInfo)
	{
		FString Path;
		FString Anchor;
		FString QueryString;
		SplitLink(Link, Path, QueryString, Anchor);

		AddSourceInfoToQueryString(QueryString, SourceInfo);

		return FString::Printf(TEXT("file:///%s%s%s"), *ToFilePath(Link, Culture), *QueryString, *Anchor);
	}

	static FString ToSourcePath(const FString& Link, const FString& BasePath = FString())
	{
		FInternationalization& I18N = FInternationalization::Get();

		FString SourcePath = ToSourcePath(Link, I18N.GetCurrentCulture(), BasePath);

		if (!FPaths::FileExists(SourcePath))
		{
			const FCulturePtr FallbackCulture = I18N.GetCulture(TEXT("en"));
			if (FallbackCulture.IsValid())
			{
				const FString FallbackSourcePath = ToSourcePath(Link, FallbackCulture.ToSharedRef(), BasePath);
				if (FPaths::FileExists(FallbackSourcePath))
				{
					SourcePath = FallbackSourcePath;
				}
			}
		}

		return SourcePath;
	}

	static FString ToSourcePath(const FString& Link, const FCultureRef& Culture, const FString& BasePath = FString())
	{
		FString Path;
		FString Anchor;
		FString QueryString;
		SplitLink(Link, Path, QueryString, Anchor);

		FString DocSourcePath = BasePath;
		if (DocSourcePath.IsEmpty() && !IDocumentation::Get()->GetSourcePaths().IsEmpty())
		{
			DocSourcePath = IDocumentation::Get()->GetSourcePaths().Last();
		}
		const FString FullDirectoryPath = FPaths::Combine(DocSourcePath, Path);

		const FString WildCard = FString::Printf(TEXT("%s/*.%s.udn"), *FullDirectoryPath, *(Culture->GetUnrealLegacyThreeLetterISOLanguageName()));

		TArray<FString> Filenames;
		IFileManager::Get().FindFiles(Filenames, *WildCard, true, false);

		if (Filenames.Num() > 0)
		{
			return FPaths::Combine(FullDirectoryPath, Filenames[0]);
		}

		// Since the source file doesn't exist already make up a valid name for a new one
		FString Category = FPaths::GetBaseFilename(Link);
		return FString::Printf(TEXT("%s/%s.%s.udn"), *FullDirectoryPath, *Category, *(Culture->GetUnrealLegacyThreeLetterISOLanguageName()));
	}

private:
	static void AddSourceInfoToQueryString(FString& QueryString, FDocumentationSourceInfo const& Info)
	{
		if (Info.IsEmpty() == false)
		{
			if (QueryString.IsEmpty())
			{
				QueryString = FString::Printf(TEXT("?utm_source=%s&utm_medium=%s&utm_campaign=%s"), *Info.Source, *Info.Medium, *Info.Campaign);
			}
			else
			{
				QueryString = FString::Printf(TEXT("%s&utm_source=%s&utm_medium=%s&utm_campaign=%s"), *QueryString, *Info.Source, *Info.Medium, *Info.Campaign);
			}
		}
	}
	
	// Splits the Link parameter into three parts:
	// - The path, which is everything up to the first ? or # character (or to the end of the link).
	// - The query string, which is everything from the first ? to the first # (or to the end of the link).
	// - The anchor, which is everything from the first # to the end of the link.
	// If the query parameter contains anything already, whatever is found in the link will
	// be appended to the existing value.
	//
	static void SplitLink( const FString& Link, /*OUT*/ FString& Path, /*OUT*/ FString& QueryString, /*OUT*/ FString& Anchor )
	{
		FString CleanedLink = Link;
		CleanedLink.TrimStartAndEndInline();

		if ( CleanedLink == TEXT("%ROOT%") )
		{
			Path.Empty();
		}
		else
		{
			FString PathAndQueryString;
			FString InAnchor = FString(Anchor).TrimChar('#');
			if ( !CleanedLink.Split( TEXT("#"), &PathAndQueryString, &Anchor ) )
			{
				PathAndQueryString = CleanedLink;
			}
			else if ( !Anchor.IsEmpty() )
			{
				// ensure leading #
				Anchor = FString( TEXT("#") ) + Anchor;
			}
			else if (!InAnchor.IsEmpty())
			{
				Anchor = FString(TEXT("#")) + InAnchor;
			}

			if ( Anchor.EndsWith( TEXT("/"), ESearchCase::CaseSensitive ) )
			{
				Anchor.LeftInline( Anchor.Len() - 1 );
			}

			if ( PathAndQueryString.EndsWith( TEXT("/"), ESearchCase::CaseSensitive ) )
			{
				PathAndQueryString.LeftInline(PathAndQueryString.Len() - 1, EAllowShrinking::No);
			}

			// split path and query string
			FString InQuery = FString(QueryString).TrimChar('?');
			if (!PathAndQueryString.Split(TEXT("?"), &Path, &QueryString))
			{
				Path = PathAndQueryString;
			}
			else if (!QueryString.IsEmpty())
			{
				// ensure leading ?
				QueryString = FString(TEXT("?")) + InQuery + QueryString;
			}
		}
	}
};
