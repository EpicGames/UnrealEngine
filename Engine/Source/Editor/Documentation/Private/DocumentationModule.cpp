// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Documentation.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "HAL/Platform.h"
#include "IDocumentation.h"
#include "IDocumentationModule.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SToolTip.h"

class SWidget;

class FDocumentationModule : public IDocumentationModule
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
		Documentation = FDocumentation::Create();

		FMultiBoxSettings::ToolTipConstructor = FMultiBoxSettings::FConstructToolTip::CreateRaw( this, &FDocumentationModule::ConstructDefaultToolTip );
	}

	virtual void ShutdownModule() override
	{
		if ( FModuleManager::Get().IsModuleLoaded("Slate") )
		{
			FMultiBoxSettings::ResetToolTipConstructor();
		}

		// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
		// we call this function before unloading the module.
	}

	virtual class TSharedRef< IDocumentation > GetDocumentation() const override
	{
		return Documentation.ToSharedRef();
	}

private:

	TSharedRef< SToolTip > ConstructDefaultToolTip( const TAttribute<FText>& ToolTipText, const TSharedPtr<SWidget>& OverrideContent, const TSharedPtr<const FUICommandInfo>& Action, bool ShowActionShortcut = false )
	{
		struct Local
		{
			/** Appends the key binding to the end of the provided ToolTip */
			static FText GetCommandShortcut(TSharedPtr< const FUICommandInfo> Command)
			{
				if (Command.IsValid() && (Command->GetFirstValidChord()->IsValidChord()))
				{
					return Command->GetInputText();
				}
				else
				{
					return FText();
				}
			}
		};

		if ( Action.IsValid() )
		{
			TAttribute<FText> Shortcut;
			if (ShowActionShortcut)
			{
				Shortcut = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&Local::GetCommandShortcut, Action));
			}

			return Documentation->CreateToolTip( ToolTipText, OverrideContent, FString( TEXT("Shared/") ) + Action->GetBindingContext().ToString(), Action->GetCommandName().ToString(), Shortcut);
		} 
		else
		{
			if (!ToolTipText.Get().IsEmpty())
			{
				TOptional<FString> TooltipNS = FTextInspector::GetNamespace(ToolTipText.Get());
				TOptional<FString> TooltipKey = FTextInspector::GetKey(ToolTipText.Get());
				if (TooltipNS.IsSet() && TooltipKey.IsSet())
				{
					return Documentation->CreateToolTip(ToolTipText, OverrideContent, FString(TEXT("Shared/MenuEntries/")) + TooltipNS.GetValue(), TooltipKey.GetValue());
				}
			}
		}

		TSharedPtr< SWidget > ToolTipContent;
        if ( OverrideContent.IsValid() )
        {
            ToolTipContent = OverrideContent;
        }
        else
        {
            ToolTipContent = SNullWidget::NullWidget;
        }
        
		return SNew( SToolTip )
			   .Text( ToolTipText )
			   [
					ToolTipContent.ToSharedRef()
			   ];
	}

private:

	TSharedPtr< IDocumentation > Documentation;

};

IMPLEMENT_MODULE( FDocumentationModule, Documentation )
