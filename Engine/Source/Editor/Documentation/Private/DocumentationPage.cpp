// Copyright Epic Games, Inc. All Rights Reserved.

#include "DocumentationPage.h"

#include "Containers/Set.h"

TSharedRef< IDocumentationPage > FDocumentationPage::Create( const FString& Link, const TSharedRef< FUDNParser >& Parser ) 
{
	return MakeShareable( new FDocumentationPage( Link, Parser ) );
}

FDocumentationPage::~FDocumentationPage() 
{

}

bool FDocumentationPage::GetExcerptContent( FExcerpt& Excerpt )
{
	if (FString* AliasName = StoredMetadata.ExcerptAliases.Find(Excerpt.Name))
	{
		if (HasExcerpt(*AliasName))
		{
			Excerpt.Name = *AliasName;
		}
	}
	for (int32 Index = 0; Index < StoredExcerpts.Num(); ++Index)
	{
		if ( Excerpt.Name == StoredExcerpts[ Index ].Name )
		{
			Parser->GetExcerptContent( Link, StoredExcerpts[ Index ] );
			Excerpt.Content = StoredExcerpts[ Index ].Content;
			Excerpt.RichText = StoredExcerpts[ Index ].RichText;
			return true;
		}
	}

	return false;
}

bool FDocumentationPage::HasExcerpt( const FString& ExcerptName )
{
	return StoredMetadata.ExcerptNames.Contains( ExcerptName );
}

int32 FDocumentationPage::GetNumExcerpts() const
{
	return StoredExcerpts.Num();
}

bool FDocumentationPage::GetExcerpt(const FString& ExcerptName, FExcerpt& Excerpt)
{
	for (const FExcerpt& StoredExcerpt : StoredExcerpts)
	{
		if (StoredExcerpt.Name == ExcerptName)
		{
			Excerpt = StoredExcerpt;
			return true;
		}
	}

	return false;
}

void FDocumentationPage::GetExcerpts( /*OUT*/ TArray< FExcerpt >& Excerpts )
{
	Excerpts.Empty();
	for (int32 i = 0; i < StoredExcerpts.Num(); ++i)
	{
		Excerpts.Add(StoredExcerpts[i]);
	}
}

FText FDocumentationPage::GetTitle()
{
	return StoredMetadata.Title;
}

void FDocumentationPage::Reload()
{
	StoredExcerpts.Empty();
	StoredMetadata = FUDNPageMetadata();
	Parser->Parse( Link, StoredExcerpts, StoredMetadata );
}

void FDocumentationPage::SetTextWrapAt( TAttribute<float> WrapAt )
{
	Parser->SetWrapAt( WrapAt );
}

FDocumentationPage::FDocumentationPage( const FString& InLink, const TSharedRef< FUDNParser >& InParser ) 
	: Link( InLink )
	, Parser( InParser )
{
	Parser->Parse( Link, StoredExcerpts, StoredMetadata );
}
