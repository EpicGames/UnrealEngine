// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IDocumentationPage.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "UDNParser.h"

class FDocumentationPage : public IDocumentationPage
{
public:

	/** Returns an instance of class containing content from the link argument */
	static TSharedRef< IDocumentationPage > Create( const FString& Link, const TSharedRef< FUDNParser >& Parser );

public:

	virtual ~FDocumentationPage();
	virtual bool HasExcerpt( const FString& ExcerptName ) override;
	virtual int32 GetNumExcerpts() const override;
	virtual bool GetExcerpt( const FString& ExcerptName, FExcerpt& Excerpt) override;
	virtual void GetExcerpts( /*OUT*/ TArray< FExcerpt >& Excerpts ) override;
	virtual bool GetExcerptContent( FExcerpt& Excerpt ) override;

	virtual FText GetTitle() override;

	virtual void Reload() override;

	virtual void SetTextWrapAt( TAttribute<float> WrapAt ) override;

private:

	FDocumentationPage( const FString& InLink, const TSharedRef< FUDNParser >& InParser );

private:

	/** The string representing the UDN page location */
	FString Link;
	/** The UDN parser instance used to create this page */
	TSharedRef< FUDNParser > Parser;

	/** The excerpts contained in this UDN page */
	TArray<FExcerpt> StoredExcerpts;
	/** The UDN meta data contained in this page */
	FUDNPageMetadata StoredMetadata;
};
