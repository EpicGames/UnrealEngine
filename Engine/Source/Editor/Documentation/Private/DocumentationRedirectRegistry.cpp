// Copyright Epic Games, Inc. All Rights Reserved.

#include "DocumentationRedirectRegistry.h"

#include "Documentation.h"
#include "UnrealEdMisc.h"

bool FDocumentationRedirectRegistry::Register(const FName& Owner, const FDocumentationRedirect& Redirect)
{
	if (!Redirect.ToUrl.IsValid())
	{
		UE_LOGF(LogDocumentation, Warning, "Documentation redirect target for \"%ls\" is not valid.", *Redirect.From);
		return false;
	}

	// Check if this redirect is from a UnrealEd URL config key
	FString FromLink;
	if (!FUnrealEdMisc::Get().GetURL(*Redirect.From, FromLink))
	{
		// Fallback on treating it as raw documentation link
		FromLink = Redirect.From;
	}

	FDocumentationRedirects& Redirects = RegisteredRedirects.FindOrAdd(FromLink);

	if (Owner.IsNone())
	{
		Redirects.Global.Add(Redirect);
	}
	else
	{
#if !UE_BUILD_SHIPPING
		ensureMsgf(!Redirects.Owned.ContainsByPredicate([&Owner](const FOwnedDocumentationRedirect& OwnedRedirect)
		{
			return OwnedRedirect.Owner == Owner;
		}), TEXT("Attempting to register a redirect for \"%s\" multiple times with the same owner name (%s). Was this intended?"), *FromLink, *Owner.ToString());
#endif

		FOwnedDocumentationRedirect OwnedRedirect;
		OwnedRedirect.Owner = Owner;
		OwnedRedirect.Redirect = Redirect;

		Redirects.Owned.Add(MoveTemp(OwnedRedirect));
	}

	UE_LOGF(LogDocumentation, Log, "[%ls] Documentation redirect registered: %ls -> %ls.", !Owner.IsNone() ? *Owner.ToString() : TEXT("Global"), *FromLink, *Redirect.ToUrl.ToString());
	
	return true;
}

void FDocumentationRedirectRegistry::UnregisterAll(const FName& Owner)
{
	for (auto RedirectsIt = RegisteredRedirects.CreateIterator(); RedirectsIt; ++RedirectsIt)
	{
		FDocumentationRedirects& Redirects = RedirectsIt->Value;

		if (Owner.IsNone())
		{
			Redirects.Global.Reset();
		}
		else
		{
			Redirects.Owned.RemoveAll([&Owner](const FOwnedDocumentationRedirect& Redirect) { return Redirect.Owner == Owner; });
		}

		// Remove mapping if there are no more redirects for this documentation link
		if (Redirects.Owned.IsEmpty() && Redirects.Global.IsEmpty())
		{
			RedirectsIt.RemoveCurrent();
		}
	}
}

bool FDocumentationRedirectRegistry::GetRedirect(const FString& Link, FDocumentationRedirect& OutRedirect) const
{
	if (const FDocumentationRedirects* Redirects = RegisteredRedirects.Find(Link))
	{
		// Owned redirects are prioritized over global redirects
		if (!Redirects->Owned.IsEmpty())
		{
			OutRedirect = Redirects->Owned.Last().Redirect;
			return true;
		}

		if (!Redirects->Global.IsEmpty())
		{
			OutRedirect = Redirects->Global.Last();
			return true;
		}
	}

	return false;
}
