// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DocumentationRedirect.h"

/**
 * Registry class to manage redirects and associate redirects with a specific documentation link
 * 
 * Every redirect is registered under an owning name to make batch registration/unregistration easier.
 * Redirects registered with "None" will be globally owned and can only be unregistered with other global redirects wholesale.
 * 
 * Documentation links with multiple redirects is supported and prioritized based on registration order.
 * The last registration is prioritized over earlier ones and prior registrations will be respected in reverse order as higher priority redirects are unregistered.
 * Redirects registered with a valid owner will always be prioritized over global redirects, regardless of registration order.
 * Since prioritization is based on registration order there is no guarantee that re-registered redirects will have the same priority.
 */
class FDocumentationRedirectRegistry
{
public:
	/**
	 * Registers a new documentation redirect for an owner
	 * @param Owner Name of the owner for the redirect
	 * @param Redirect Redirect to register
	 * @return Was the redirect successfully registered?
	 */
	bool Register(const FName& Owner, const FDocumentationRedirect& Redirect);

	/**
	 * Unregisters all redirects for an owner
	 * @param Owner Name of the owner
	 */
	void UnregisterAll(const FName& Owner);

	/**
	 * Retrieves the highest priority redirect for a documentation link
	 * @param Link Documentation link to get redirect for
	 * @param OutRedirect Redirect for the documentation link
	 * @return Was there a valid redirect for the documentation link?
	 */
	bool GetRedirect(const FString& Link, FDocumentationRedirect& OutRedirect) const;

private:
	/** Documentation redirect with ownership information */
	struct FOwnedDocumentationRedirect
	{
		FName Owner;
		FDocumentationRedirect Redirect;
	};

	/** Container of owned and global redirects for a documentation link */
	struct FDocumentationRedirects
	{
		TArray<FOwnedDocumentationRedirect> Owned;
		TArray<FDocumentationRedirect> Global;
	};

	/** Mapping between documentation link to registered redirects */
	TMap<FString, FDocumentationRedirects> RegisteredRedirects;
};
