// Copyright Epic Games, Inc. All Rights Reserved.

#include "DocumentationStyleSet.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"

FName FDocumentationStyleSet::StyleName("FDocumentationStyleSet");
TUniquePtr<FDocumentationStyleSet> FDocumentationStyleSet::Instance(nullptr);

const FName& FDocumentationStyleSet::GetStyleSetName() const
{
	return StyleName;
}

const FDocumentationStyleSet& FDocumentationStyleSet::Get()
{
	if (!Instance.IsValid())
	{
		Instance = TUniquePtr<FDocumentationStyleSet>(new FDocumentationStyleSet);
	}
	return *(Instance.Get());
}

void FDocumentationStyleSet::Shutdown()
{
	Instance.Reset();
}

FDocumentationStyleSet::FDocumentationStyleSet()
	: FSlateStyleSet(StyleName)
{
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	const FVector2D Icon128x128(128.0f, 128.0f);

	const FLinearColor DarkHeaderBackground = COLOR("#383838FF");        // Dropdown
	const FLinearColor ContentBackground = COLOR("#4F4F4FFF");           // New shade
	const FLinearColor ItemBackground = COLOR("#808080FF");              // Hover2
	const FLinearColor ItemBackgroundHighlight = COLOR("#A0A0A0FF");     // New shade between Hover2 and Foreground

	SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	Set("ToolTip.Header", new FSlateColorBrush(DarkHeaderBackground));

	Set("ToolTip.ScrollBox", FScrollBoxStyle()
		.SetTopShadowBrush(FSlateNoResource())
		.SetBottomShadowBrush(FSlateNoResource())
	);
	Set("ToolTip.Scrollbar", FScrollBarStyle(FAppStyle::GetWidgetStyle<FScrollBarStyle>("ScrollBar"))
		.SetNormalThumbImage(FSlateRoundedBoxBrush(ItemBackground, 4.0f))
		.SetDraggedThumbImage(FSlateRoundedBoxBrush(ItemBackgroundHighlight, 4.0f))
		.SetHoveredThumbImage(FSlateRoundedBoxBrush(ItemBackgroundHighlight, 4.0f))
		.SetVerticalBackgroundImage(FSlateColorBrush(DarkHeaderBackground))
		.SetHorizontalBackgroundImage(FSlateColorBrush(DarkHeaderBackground))
	);
	Set("ToolTip.ContentBackground", new FSlateColorBrush(ContentBackground));

	Set("ToolTip.TopSeparator", new BORDER_BRUSH("Common/Selector", FMargin(0.0f, 1.0f, 0.0f, 0.0f), FStyleColors::Black));

	Set("ToolTip.KeybindBorder", new FSlateRoundedBoxBrush(ItemBackground, 4.0f));
	const FTextBlockStyle KeybindText = FTextBlockStyle(FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.SetFont(DEFAULT_FONT("Bold", 10))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f));
	Set("ToolTip.KeybindText", FTextBlockStyle(KeybindText));

	Set("ToolTip.ToggleKeybindBorder", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FLinearColor(0.1f, 0.1f, 0.1f), 1.0f));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FDocumentationStyleSet::~FDocumentationStyleSet()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
