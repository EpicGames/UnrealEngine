// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

class FDocumentationStyleSet :
    public FSlateStyleSet
{
public:
	virtual const FName& GetStyleSetName() const override;

	static const FDocumentationStyleSet& Get();
	static void Shutdown();

	~FDocumentationStyleSet();

private:
	FDocumentationStyleSet();

	static FName StyleName;
	static TUniquePtr<FDocumentationStyleSet> Instance;
};


