// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDocumentationAnchor.h"

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "IDocumentation.h"
#include "IDocumentationPage.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "SSimpleButton.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/SToolTip.h"

void SDocumentationAnchor::Construct(const FArguments& InArgs )
{
	Link = InArgs._Link;
	BaseUrlId = InArgs._BaseUrlId;

	SetVisibility(TAttribute<EVisibility>::CreateLambda([this]()
		{
			return Link.Get(FString()).IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
		}));

	TAttribute<FText> ToolTipText = InArgs._ToolTipText;
	if (!ToolTipText.IsBound() && ToolTipText.Get().IsEmpty())
	{
		ToolTipText = NSLOCTEXT("DocumentationAnchor", "DefaultToolTip", "Click to open documentation");
	}

	const FString PreviewLink = InArgs._PreviewLink;
	const FString PreviewExcerptName = InArgs._PreviewExcerptName;
	if (!PreviewLink.IsEmpty() && !PreviewExcerptName.IsEmpty() && BaseUrlId.Get().IsEmpty())
	{
		TSharedRef<IDocumentation> Documentation = IDocumentation::Get();
		if (Documentation->PageExists(PreviewLink))
		{
			TSharedPtr<IDocumentationPage> DocumentationPage = Documentation->GetPage(PreviewLink, NULL);

			FExcerpt Excerpt;
			if (DocumentationPage->HasExcerpt(PreviewExcerptName))
			{
				if (DocumentationPage->GetExcerpt(PreviewExcerptName, Excerpt))
				{
					if (FString* BaseUrlValue = Excerpt.Variables.Find(TEXT("BaseUrl")))
					{
						BaseUrlId = *BaseUrlValue;
					}
				}
			}
		}

	}

	ChildSlot
	[
		SAssignNew(Button, SSimpleButton)
		.OnClicked(this, &SDocumentationAnchor::OnClicked)
		.Icon(FAppStyle::Get().GetBrush("Icons.Help"))
		.ToolTip(IDocumentation::Get()->CreateToolTip(ToolTipText, nullptr, PreviewLink, PreviewExcerptName))
	];
}


FReply SDocumentationAnchor::OnClicked() const
{
	IDocumentation::Get()->Open(Link.Get(FString()), FDocumentationSourceInfo(TEXT("doc_anchors")), BaseUrlId.Get(FString()));
	return FReply::Handled();
}
