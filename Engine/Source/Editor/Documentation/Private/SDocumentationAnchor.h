// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Input/Reply.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SDocumentationAnchor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SDocumentationAnchor )
	{}
		SLATE_ARGUMENT( FString, PreviewLink )
		SLATE_ARGUMENT( FString, PreviewExcerptName )

		/** The string for the link to follow when clicked  */
		SLATE_ATTRIBUTE( FString, Link )
		/** The base URL for the Link, if any is needed  */
		SLATE_ATTRIBUTE(FString, BaseUrlId)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	
	FReply OnClicked() const;

private:

	TAttribute<FString> Link;
	TAttribute<FString> BaseUrlId;
	TSharedPtr<class SButton> Button;
	TSharedPtr<class SImage> ButtonImage;
};
