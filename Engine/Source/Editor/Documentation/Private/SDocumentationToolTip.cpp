// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDocumentationToolTip.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SToolTip.h"
#include "Styling/AppStyle.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "IDocumentationPage.h"
#include "IDocumentation.h"
#include "DocumentationLink.h"
#include "ISourceCodeAccessor.h"
#include "ISourceCodeAccessModule.h"
#include "SourceControlHelpers.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Widgets/Input/SHyperlink.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "DocumentationStyleSet.h"


namespace UE::UnrealEd::Private
{
	float CVarTooltipExpandTimeValue = 0.12f;

	// The visual width of the outside border around the tooltip contents.
	static float WidgetContentPadding = 1.5f;
}

static FAutoConsoleVariableRef CVarTooltipExpandTime(
	TEXT("TooltipExpandTime"),
	UE::UnrealEd::Private::CVarTooltipExpandTimeValue,
	TEXT("Set the transition time for tooltip expansion."),
	ECVF_Default
);

void SDocumentationToolTip::Construct( const FArguments& InArgs )
{
	TextContent = InArgs._Text;
	StyleInfo = FAppStyle::GetWidgetStyle<FTextBlockStyle>(InArgs._Style);
	SubduedStyleInfo = FAppStyle::GetWidgetStyle<FTextBlockStyle>(InArgs._SubduedStyle);
	HyperlinkTextStyleInfo = FAppStyle::GetWidgetStyle<FTextBlockStyle>(InArgs._HyperlinkTextStyle);
	HyperlinkButtonStyleInfo = FAppStyle::GetWidgetStyle<FButtonStyle>(InArgs._HyperlinkButtonStyle);
	KeybindStyleInfo = FDocumentationStyleSet::Get().GetWidgetStyle<FTextBlockStyle>("ToolTip.KeybindText");
	ColorAndOpacity = InArgs._ColorAndOpacity;
	DocumentationLink = InArgs._DocumentationLink;
	bAddDocumentation = InArgs._AddDocumentation;
	DocumentationMargin = InArgs._DocumentationMargin;
	IsDisplayingDocumentationLink = false;
	Shortcut = InArgs._Shortcut;
	OverrideFullTooltipContent = InArgs._OverrideExtendedToolTipContent;
	OverridePromptContent = InArgs._OverridePromptContent;
	AlwaysExpandTooltip = InArgs._AlwaysExpandTooltip;

	ExcerptName = InArgs._ExcerptName;
	IsShowingFullTip = false;

	if( InArgs._Content.Widget != SNullWidget::NullWidget )
	{
		// Widget content argument takes precedence
		// overrides the text content.
		OverrideContent = InArgs._Content.Widget;
	}

	SAssignNew(SimpleTipContent, SBox);
	SAssignNew(DocumentationControls, SHorizontalBox);
	SAssignNew(FullTipContent, SBox);
	if (OverrideFullTooltipContent.IsValid())
	{
		FullTipContent->SetContent(OverrideFullTooltipContent.ToSharedRef());
		FullTipContent->SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SDocumentationToolTip::GetOverriddenFullToolTipVisibility));
	}

	ConstructSimpleTipContent();
	ChildSlot
	[
		SAssignNew(WidgetContent, SBox)
		.Padding(UE::UnrealEd::Private::WidgetContentPadding)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			[
				SimpleTipContent.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				FullTipContent.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(DocumentationControlBox, SBox)
				.Visibility(this, &SDocumentationToolTip::GetControlVisibility)
				[
					SNew(SBorder)
					.Padding(0,1,0,0)
					.BorderImage(FDocumentationStyleSet::Get().GetBrush("ToolTip.TopSeparator"))
					[
						SNew(SBorder)
						.Padding(9)
						.BorderImage(FDocumentationStyleSet::Get().GetBrush("ToolTip.Header"))
						[
							DocumentationControls.ToSharedRef()
						]
					]
				]
			]
		]
	];

	TransitionStartTime = 0;
	bFullTipContentIsReady = OverrideFullTooltipContent.IsValid();
	bIsPromptVisible = !IsShowingFullTip;
}

void SDocumentationToolTip::ConstructSimpleTipContent()
{
	// If there a UDN file that matches the DocumentationLink path, and that page has an excerpt whose name
	// matches ExcerptName, and that excerpt has a variable named ToolTipOverride, use the content of that
	// variable instead of the default TextContent.
	if (!DocumentationLink.IsEmpty() && !ExcerptName.IsEmpty())
	{
		TSharedRef<IDocumentation> Documentation = IDocumentation::Get();
		if (Documentation->PageExists(DocumentationLink))
		{
			DocumentationPage = Documentation->GetPage(DocumentationLink, NULL);

			FExcerpt Excerpt;
			if (DocumentationPage->HasExcerpt(ExcerptName))
			{
				if (DocumentationPage->GetExcerpt(ExcerptName, Excerpt))
				{
					if (FString* TooltipValue = Excerpt.Variables.Find(TEXT("ToolTipOverride")))
					{
						TextContent = FText::FromString(*TooltipValue);
					}
				}
			}
		}
	}

	TSharedPtr< SVerticalBox > VerticalBox;
	TSharedPtr< SHorizontalBox > TextBox;
	if ( !OverrideContent.IsValid() )
	{
		SimpleTipContent->SetContent(
			SNew(SBorder)
			.Padding(9.f)
			.BorderImage(FAppStyle::GetNoBrush())
			[
				SAssignNew( VerticalBox, SVerticalBox )
				+SVerticalBox::Slot()
				.FillHeight( 1.0f )
				[
					SAssignNew(TextBox, SHorizontalBox)

					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
					SNew( STextBlock )
					.Text( TextContent )
					.TextStyle( &StyleInfo )
					.ColorAndOpacity( ColorAndOpacity )
					.WrapTextAt_Static( &SToolTip::GetToolTipWrapWidth )
					]
				]
			]
		);

		TextBox->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
			.Visibility(this, &SDocumentationToolTip::GetShortcutVisibility)
			.Padding(9.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
				.BorderImage(FDocumentationStyleSet::Get().GetBrush("ToolTip.KeybindBorder"))
				.Padding(4.0f, 2.0f)
				[
					SNew(STextBlock)
					.TextStyle(&KeybindStyleInfo)
					.Text(Shortcut)
				]
			]
		];
	
	}
	else
	{
		SimpleTipContent->SetContent(
			SNew(SBorder)
			.Padding(9.f)
			.BorderImage(FAppStyle::GetNoBrush())
			[
				SAssignNew( VerticalBox, SVerticalBox )
				+SVerticalBox::Slot()
				.FillHeight( 1.0f )
				[
					OverrideContent.ToSharedRef()
				]
			]
		);
	}

	if (bAddDocumentation)
	{
		AddDocumentation(VerticalBox);
	}
}

void SDocumentationToolTip::AddDocumentation(TSharedPtr< SVerticalBox > VerticalBox)
{
	if ( !DocumentationLink.IsEmpty() || OverrideFullTooltipContent.IsValid() )
	{
		if ( !DocumentationPage.IsValid() )
		{
			DocumentationPage = IDocumentation::Get()->GetPage( DocumentationLink, NULL );
		}

		if ( DocumentationPage->HasExcerpt( ExcerptName ) || OverrideFullTooltipContent.IsValid())
		{
			if (OverridePromptContent.IsValid())
			{
				VerticalBox->AddSlot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(SBox)
						.Visibility(this, &SDocumentationToolTip::GetPromptVisibility)
						.HeightOverride(this, &SDocumentationToolTip::GetPromptHeight)
						[
							SAssignNew(PromptContent,SBox)
							[
								OverridePromptContent.ToSharedRef()
							]
						]
					];
			}
			else
			{
				FText MacShortcut = NSLOCTEXT("SToolTip", "MacRichTooltipShortcut", "Command + Option");
				FText WinShortcut = NSLOCTEXT("SToolTip", "WinRichTooltipShortcut", "Ctrl + Alt");

				FText KeyboardShortcut;
#if PLATFORM_MAC
				KeyboardShortcut = MacShortcut;
#else
				KeyboardShortcut = WinShortcut;
#endif

				VerticalBox->AddSlot()
				.AutoHeight()
				.HAlign( HAlign_Center )
				[
					SNew(SBox)
					.Visibility(this, &SDocumentationToolTip::GetPromptVisibility)
					.HeightOverride(this, &SDocumentationToolTip::GetPromptHeight)
					[
						SAssignNew(PromptContent, SBox)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.Padding(0, 5, 0, 0)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.AutoWidth()
							[
								SNew( STextBlock )
								.TextStyle( &SubduedStyleInfo )
								.Text( NSLOCTEXT( "SToolTip", "AdvancedToolTipMessage", "Hold" ) )
							]
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							.Padding(5, 5, 5, 0)
							[
								SNew( SBorder )
								.BorderImage(FDocumentationStyleSet::Get().GetBrush("ToolTip.ToggleKeybindBorder"))
								.Padding(4, 2)
								[
									SNew(STextBlock)
									.TextStyle(&SubduedStyleInfo)
									.Text(FText::Format(NSLOCTEXT("SToolTip", "AdvancedToolTipKeybind", "{0}"), KeyboardShortcut))
									.Visibility(this, &SDocumentationToolTip::GetPromptVisibility)
								]
							]
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.Padding(0, 5, 0, 0)
							.AutoWidth()
							[
								SNew(STextBlock)
									.TextStyle(&SubduedStyleInfo)
									.Text(NSLOCTEXT("SToolTip", "AdvancedToolTipMessageEnd", "for more"))
							]
						]
					]
				];
			}
		}

		IsDisplayingDocumentationLink = GetDefault<UEditorPerProjectUserSettings>()->bDisplayDocumentationLink;
		if (IsDisplayingDocumentationLink)
		{

			FString OptionalExcerptName;
			if (!ExcerptName.IsEmpty())
			{
				OptionalExcerptName = FString(TEXT(" [")) + ExcerptName + TEXT("]");
			}
			DocumentationControls->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Left)
			.Padding(0, 0, 9, 0)
			[
				SNew(STextBlock)
					.Text(FText::FromString(DocumentationLink + OptionalExcerptName))
					.TextStyle(FAppStyle::Get(), "Documentation.Text")
			];

			if (!DocumentationPage->HasExcerpt(ExcerptName) && FSlateApplication::Get().SupportsSourceAccess())
			{
				FString DocPath = FDocumentationLink::ToSourcePath(DocumentationLink, FInternationalization::Get().GetCurrentCulture());
				if (!FPaths::FileExists(DocPath))
				{
					DocPath = FPaths::ConvertRelativePathToFull(DocPath);
				}

				DocumentationControls->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					SNew(SHyperlink)
						.Text(NSLOCTEXT("SToolTip", "EditDocumentationMessage_Create", "create"))
						.TextStyle(&HyperlinkTextStyleInfo)
						.UnderlineStyle(&HyperlinkButtonStyleInfo)
						.OnNavigate(this, &SDocumentationToolTip::CreateExcerpt, DocPath, ExcerptName)
				];
			}
		}
	}
}

void SDocumentationToolTip::CreateExcerpt( FString FileSource, FString InExcerptName )
{
	FText CheckoutFailReason;
	bool bNewFile = true;
	bool bCheckoutOrAddSucceeded = true;
	if (FPaths::FileExists(FileSource))
	{
		// Check out the existing file
		bNewFile = false;
		bCheckoutOrAddSucceeded = SourceControlHelpers::CheckoutOrMarkForAdd(FileSource, NSLOCTEXT("SToolTip", "DocumentationSCCActionDesc", "tool tip excerpt"), FOnPostCheckOut(), /*out*/ CheckoutFailReason);
	}

	FArchive* FileWriter = IFileManager::Get().CreateFileWriter( *FileSource, EFileWrite::FILEWRITE_Append | EFileWrite::FILEWRITE_AllowRead | EFileWrite::FILEWRITE_EvenIfReadOnly );

	if (bNewFile)
	{
		FString UdnHeader;
		UdnHeader += "Availability:NoPublish";
		UdnHeader += LINE_TERMINATOR;
		UdnHeader += "Title:";
		UdnHeader += LINE_TERMINATOR;
		UdnHeader += "Crumbs:";
		UdnHeader += LINE_TERMINATOR;
		UdnHeader += "Description:";
		UdnHeader += LINE_TERMINATOR;

		FileWriter->Serialize( TCHAR_TO_ANSI( *UdnHeader ), UdnHeader.Len() );
	}

	FString NewExcerpt;
	NewExcerpt += LINE_TERMINATOR;
	NewExcerpt += "[EXCERPT:";
	NewExcerpt += InExcerptName;
	NewExcerpt += "]";
	NewExcerpt += LINE_TERMINATOR;

	NewExcerpt += TextContent.Get().ToString();
	NewExcerpt += LINE_TERMINATOR;

	NewExcerpt += "[/EXCERPT:";
	NewExcerpt += InExcerptName;
	NewExcerpt += "]";
	NewExcerpt += LINE_TERMINATOR;

	if (!bNewFile)
	{
		FileWriter->Seek( FMath::Max( FileWriter->TotalSize(), (int64)0 ) );
	}

	FileWriter->Serialize( TCHAR_TO_ANSI( *NewExcerpt ), NewExcerpt.Len() );

	FileWriter->Close();
	delete FileWriter;

	if (bNewFile)
	{
		// Add the new file
		bCheckoutOrAddSucceeded = SourceControlHelpers::CheckoutOrMarkForAdd(FileSource, NSLOCTEXT("SToolTip", "DocumentationSCCActionDesc", "tool tip excerpt"), FOnPostCheckOut(), /*out*/ CheckoutFailReason);
	}

	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	SourceCodeAccessModule.GetAccessor().OpenFileAtLine(FileSource, 0);

	if (!bCheckoutOrAddSucceeded)
	{
		FNotificationInfo Info(CheckoutFailReason);
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}

	ReloadDocumentation();
}

void SDocumentationToolTip::ConstructFullTipContent()
{
	TArray< FExcerpt > Excerpts;
	DocumentationPage->GetExcerpts( Excerpts );

	if ( Excerpts.Num() > 0 )
	{
		int32 ExcerptIndex = 0;
		if ( !ExcerptName.IsEmpty() )
		{
			for (int Index = 0; Index < Excerpts.Num(); Index++)
			{
				if ( Excerpts[ Index ].Name == ExcerptName )
				{
					ExcerptIndex = Index;
					break;
				}
			}
		}

		if ( !Excerpts[ ExcerptIndex ].Content.IsValid() )
		{
			DocumentationPage->GetExcerptContent( Excerpts[ ExcerptIndex ] );
		}

		if ( Excerpts[ ExcerptIndex ].Content.IsValid() )
		{
			TSharedPtr< SVerticalBox > Box;
			TSharedPtr< SWidget > FullTipBox = SNew(SBox)
			.Padding(DocumentationMargin)
			.Visibility(this, &SDocumentationToolTip::GetFullTipVisibility)
			[
				SNew(SBorder)
				.BorderImage(FDocumentationStyleSet::Get().GetBrush("ToolTip.TopSeparator"))
				.Padding(0,1,0,0)
				[
					SNew(SBorder)
					.BorderImage(FDocumentationStyleSet::Get().GetBrush("ToolTip.ContentBackground"))
					.Padding(0)
					[
						SAssignNew(Box, SVerticalBox)
						+ SVerticalBox::Slot()
						.HAlign(HAlign_Center)
						.MaxHeight(750)
						[
							SNew(SScrollBox)
							.Style(FDocumentationStyleSet::Get(), "ToolTip.ScrollBox")
							.ScrollBarStyle(FDocumentationStyleSet::Get(), "ToolTip.ScrollBar")
							+SScrollBox::Slot()
							.Padding(14.0f)
							.FillSize(1.0f)
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								[
									Excerpts[ExcerptIndex].Content.ToSharedRef()
								]
							]
						]
					]
				]
			];

			FString* FullDocumentationLink = Excerpts[ ExcerptIndex ].Variables.Find( TEXT("ToolTipFullLink") );
			FString* ExcerptBaseUrl = Excerpts[ExcerptIndex].Variables.Find(TEXT("BaseUrl"));
			if ( FullDocumentationLink != NULL && !FullDocumentationLink->IsEmpty() )
			{
				FString BaseUrl = FString();
				if (ExcerptBaseUrl != NULL)
				{
					BaseUrl = *ExcerptBaseUrl;
				}

				Box->AddSlot()
				.AutoHeight()
				[
					SNew(SBorder)
					.Padding(9.0f)
					.BorderImage(FDocumentationStyleSet::Get().GetBrush("ToolTip.Header"))
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						[
							SNew(SButton)
							.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
							.ContentPadding(FMargin(0.0f, 4.5f))
							.Content()
							[
								SNew(SHorizontalBox)
								+SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.AutoWidth()
								.Padding(0,0,3,0)
								[
									SNew(SImage)
									.Image(FAppStyle::GetBrush("Icons.Help.Solid"))
								]
								+SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.AutoWidth()
								[
									SNew(STextBlock)
									.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("PrimaryButtonText"))
									.Text(NSLOCTEXT("SToolTip", "LearnMoreButton", "Learn More Online"))
								]
							]

							.OnClicked_Static([](FString Link, FString BaseUrl) -> FReply {
								if (!IDocumentation::Get()->Open(Link, FDocumentationSourceInfo(TEXT("rich_tooltips")), BaseUrl))
								{
									FNotificationInfo Info(NSLOCTEXT("SToolTip", "FailedToOpenLink", "Failed to Open Link"));
									FSlateNotificationManager::Get().AddNotification(Info);
								}
								return FReply::Handled();
								}, *FullDocumentationLink, BaseUrl)
						]
					]
				];
			}

			if (IsDisplayingDocumentationLink && FSlateApplication::Get().SupportsSourceAccess() )
			{
				DocumentationControls->AddSlot()
				.AutoWidth()
				.HAlign( HAlign_Right )
				[
					SNew( SHyperlink )
					.Text( NSLOCTEXT( "SToolTip", "EditDocumentationMessage_Edit", "edit" ) )
					.TextStyle( &HyperlinkTextStyleInfo )
					.UnderlineStyle( &HyperlinkButtonStyleInfo )
					// todo: needs to update to point to the "real" source file used for the excerpt
					.OnNavigate_Static([](FString Link, int32 LineNumber) {
							ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
							SourceCodeAccessModule.GetAccessor().OpenFileAtLine(Link, LineNumber);
						}, FPaths::ConvertRelativePathToFull(FDocumentationLink::ToSourcePath(DocumentationLink, FInternationalization::Get().GetCurrentCulture())), Excerpts[ExcerptIndex].LineNumber)
				];
			}
			
			FullTipContent->SetContent(FullTipBox.ToSharedRef());
			bFullTipContentIsReady = true;
		}
	}
}

FReply SDocumentationToolTip::ReloadDocumentation()
{
	bFullTipContentIsReady = false;

	DocumentationControls->ClearChildren();
	ConstructSimpleTipContent();

	if ( DocumentationPage.IsValid() )
	{
		DocumentationPage->Reload();

		if ( DocumentationPage->HasExcerpt( ExcerptName ) )
		{
			ConstructFullTipContent();
		}
	}

	return FReply::Handled();
}

void SDocumentationToolTip::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	const bool NeedsUpdate = !OverrideFullTooltipContent.IsValid() && IsDisplayingDocumentationLink != GetDefault<UEditorPerProjectUserSettings>()->bDisplayDocumentationLink;
	if (TransitionStartTime > 0)
	{
		const float TransitionPercentage = (InCurrentTime - TransitionStartTime) / TransitionLength;
		if (TransitionPercentage >= 1.0f)
		{
			// Stop transition.
			TransitionStartTime = 0;
		}
	}

	if ( !IsShowingFullTip && ((ModifierKeys.IsAltDown() && ModifierKeys.IsControlDown()) || ShouldAlwaysExpand()) )
	{
		if (!OverrideFullTooltipContent.IsValid())
		{
			if ( !bFullTipContentIsReady && DocumentationPage.IsValid() && DocumentationPage->HasExcerpt(ExcerptName))
			{
				ConstructFullTipContent();
			}
			else if ( GetDefault<UEditorPerProjectUserSettings>()->bDisplayDocumentationLink )
			{
				ReloadDocumentation();
			}
		}

		if ( bFullTipContentIsReady)
		{
			if (!OverrideFullTooltipContent.IsValid())
			{
				// Analytics event
				if (FEngineAnalytics::IsAvailable())
				{
					TArray<FAnalyticsEventAttribute> Params;
					Params.Add(FAnalyticsEventAttribute(TEXT("Page"), DocumentationLink));
					Params.Add(FAnalyticsEventAttribute(TEXT("Excerpt"), ExcerptName));

					FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Documentation.FullTooltipShown"), Params);
				}
			}
			bIsPromptVisible = false;
			TransitionStartTime = InCurrentTime;
			TransitionLength = UE::UnrealEd::Private::CVarTooltipExpandTimeValue;
		}
		IsShowingFullTip = true;
	}
	else if ( ( IsShowingFullTip || NeedsUpdate )  && (( !ModifierKeys.IsAltDown() || !ModifierKeys.IsControlDown() ) && !ShouldAlwaysExpand() ))
	{
		if ( NeedsUpdate )
		{
			IsDisplayingDocumentationLink = GetDefault<UEditorPerProjectUserSettings>()->bDisplayDocumentationLink;
			ReloadDocumentation();
		}

		IsShowingFullTip = false;
		bIsPromptVisible = true;
	}
}

bool SDocumentationToolTip::IsInteractive() const
{
	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	return ((OverrideFullTooltipContent.IsValid() || DocumentationPage.IsValid()) && ModifierKeys.IsAltDown() && ModifierKeys.IsControlDown() );
}

FVector2D SDocumentationToolTip::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	FVector2D SimpleTipSize = SimpleTipContent->GetDesiredSize();
	FVector2D TransitionStartSize = SimpleTipSize + (2 * UE::UnrealEd::Private::WidgetContentPadding);
	if (GetControlVisibility() == EVisibility::Visible)
	{
		TransitionStartSize.Y += DocumentationControlBox->GetDesiredSize().Y;
		TransitionStartSize.X = FMath::Max(TransitionStartSize.X, DocumentationControlBox->GetDesiredSize().X + (2 * UE::UnrealEd::Private::WidgetContentPadding));
	}
	if (TransitionStartTime > 0 && !IsDisplayingDocumentationLink)
	{
		const float TransitionPercentage = FMath::Clamp(((FSlateApplication::Get().GetCurrentTime() - TransitionStartTime) / TransitionLength), 0, 1);

		const FVector2D TransitionEndSize = (IsShowingFullTip) ? WidgetContent->GetDesiredSize() : TransitionStartSize;
		return (TransitionStartSize
			- ((TransitionStartSize - TransitionEndSize)
				* FMath::InterpEaseOut<float>(0.f, 1.f, TransitionPercentage, 4.f)));
	}

	return (IsShowingFullTip) ? WidgetContent->GetDesiredSize() : TransitionStartSize;
}

EVisibility SDocumentationToolTip::GetOverriddenFullToolTipVisibility() const
{
	return IsShowingFullTip ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDocumentationToolTip::GetPromptVisibility() const
{
	return (bIsPromptVisible) ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility SDocumentationToolTip::GetFullTipVisibility() const
{
	return (IsShowingFullTip) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDocumentationToolTip::GetControlVisibility() const
{
	if (IsDisplayingDocumentationLink && (IsShowingFullTip || !DocumentationPage.IsValid() || !DocumentationPage->HasExcerpt(ExcerptName)) && !OverrideFullTooltipContent.IsValid())
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

EVisibility SDocumentationToolTip::GetShortcutVisibility() const
{
	if ((Shortcut.IsSet() || Shortcut.IsBound()) && !Shortcut.Get().IsEmpty())
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

bool SDocumentationToolTip::ShouldAlwaysExpand() const
{
	return AlwaysExpandTooltip.IsSet() && AlwaysExpandTooltip.Get();
}

FOptionalSize SDocumentationToolTip::GetPromptHeight() const
{
	const float DesiredHeight = PromptContent->GetDesiredSize().Y;
	if (IsShowingFullTip)
	{
		if (TransitionStartTime > 0)
		{
			const float TransitionPercentage = FMath::Clamp(((FSlateApplication::Get().GetCurrentTime() - TransitionStartTime) / (0.75 * TransitionLength)), 0, 1);
			return DesiredHeight * (1 - FMath::InterpEaseOut<float>(0.f, 1.f, TransitionPercentage, 4.f));
		}
		return 0.0f;
	}
	return DesiredHeight;
}

