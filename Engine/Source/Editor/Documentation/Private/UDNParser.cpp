// Copyright Epic Games, Inc. All Rights Reserved.

#include "UDNParser.h"

#include "Application/SlateApplicationBase.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "ContentBrowserModule.h"
#include "Delegates/Delegate.h"
#include "DocumentationLink.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/SlateDelegates.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "IContentBrowserSingleton.h"
#include "IDocumentationPage.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Logging/MessageLog.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "MessageLogModule.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Char.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Rendering/SlateRenderer.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/StyleColors.h"

class SWidget;

#define LOCTEXT_NAMESPACE "IntroTutorials"

FName UDNParseErrorLog("UDNParser");

namespace LinkPrefixes
{

static const FString DocLinkSpecifier( TEXT( "DOCLINK:" ) );
static const FString HttpLinkSpecifier( TEXT( "http://" ) );
static const FString HttpsLinkSpecifier( TEXT( "https://" ) );

static const FString CodeLinkSpecifier(TEXT("CODELINK:"));
static const FString AssetLinkSpecifier(TEXT("ASSETLINK:"));

}

static const int IndentWidth = 32;

TSharedRef< FUDNParser > FUDNParser::Create( const TSharedPtr< FParserConfiguration >& ParserConfig, const FDocumentationStyle& Style ) 
{
	TSharedPtr< FParserConfiguration > FinalParserConfig = ParserConfig;
	if ( !FinalParserConfig.IsValid() )
	{
		struct Local
		{
			static void OpenLink( const FString& Link )
			{
				if ( !IDocumentation::Get()->Open( Link, FDocumentationSourceInfo(TEXT("udn_parser")) ) )
				{
					FNotificationInfo Info( NSLOCTEXT("FUDNParser", "FailedToOpenLink", "Failed to Open Link") );
					FSlateNotificationManager::Get().AddNotification(Info);
				}
			}
		};

		FinalParserConfig = FParserConfiguration::Create();
		FinalParserConfig->OnNavigate = FOnNavigate::CreateStatic( &Local::OpenLink );
	}

	TSharedRef< FUDNParser > Parser = MakeShareable( new FUDNParser( FinalParserConfig.ToSharedRef(), Style ) );
	Parser->Initialize();
	return Parser;
}

FUDNParser::FUDNParser( const TSharedRef< FParserConfiguration >& InConfiguration, const FDocumentationStyle& InStyle )
	: Configuration( InConfiguration )
	, Style(InStyle)
	, WrapAt(450.f)
	, ContentWidth(450.f)
{

}

void FUDNParser::Initialize()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing( UDNParseErrorLog, LOCTEXT("UDNParser", "UDN Parse Errors") );

	// Set up rules for interpreting strings as tokens
	TokenLibrary.Add(FTokenPair(TEXT("#"), EUDNToken::Pound));
	TokenLibrary.Add(FTokenPair(TEXT("["), EUDNToken::OpenBracket));
	TokenLibrary.Add(FTokenPair(TEXT("]"), EUDNToken::CloseBracket));
	TokenLibrary.Add(FTokenPair(TEXT("("), EUDNToken::OpenParenthesis));
	TokenLibrary.Add(FTokenPair(TEXT(")"), EUDNToken::CloseParenthesis));
	TokenLibrary.Add(FTokenPair(TEXT("1."), EUDNToken::Numbering));
	TokenLibrary.Add(FTokenPair(TEXT("!"), EUDNToken::Bang));
	TokenLibrary.Add(FTokenPair(TEXT("EXCERPT"), EUDNToken::Excerpt));
	TokenLibrary.Add(FTokenPair(TEXT("VAR"), EUDNToken::Variable));
	TokenLibrary.Add(FTokenPair(TEXT(":"), EUDNToken::Colon));
	TokenLibrary.Add(FTokenPair(TEXT("/"), EUDNToken::Slash));
	TokenLibrary.Add(FTokenPair(TEXT("-"), EUDNToken::Dash));
	TokenLibrary.Add(FTokenPair(TEXT("Availability:"), EUDNToken::MetadataAvailability));
	TokenLibrary.Add(FTokenPair(TEXT("Title:"), EUDNToken::MetadataTitle));
	TokenLibrary.Add(FTokenPair(TEXT("Crumbs:"), EUDNToken::MetadataCrumbs));
	TokenLibrary.Add(FTokenPair(TEXT("Description:"), EUDNToken::MetadataDescription));
	TokenLibrary.Add(FTokenPair(TEXT("BaseUrl:"), EUDNToken::MetadataBaseUrl));
	TokenLibrary.Add(FTokenPair(TEXT("%"), EUDNToken::Percentage));
	TokenLibrary.Add(FTokenPair(TEXT("*"), EUDNToken::Asterisk));

	// Set up rules for interpreting series of symbols into a line of Slate content
	TArray<EUDNToken::Type> TokenArray;
	
	TokenArray.Empty();
	TokenArray.Add(EUDNToken::Asterisk);
	TokenArray.Add(EUDNToken::Asterisk);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::Asterisk);
	TokenArray.Add(EUDNToken::Asterisk);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::BoldContent));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::Percentage);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::Percentage);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::VariableReference));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::Numbering);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::NumberedContent, true));
	
	TokenArray.Empty();
	for (int32 i = 0; i < 3; ++i)
	{
		TokenArray.Add(EUDNToken::Dash);
	}
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::HorizontalRule));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::Dash);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::BulletContent, true));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::Pound);
	TokenArray.Add(EUDNToken::Pound);
	TokenArray.Add(EUDNToken::Pound);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::Header3, true));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::Pound);
	TokenArray.Add(EUDNToken::Pound);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::Header2, true));
	
	TokenArray.Empty();
	TokenArray.Add(EUDNToken::Pound);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::Header1, true));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::OpenBracket);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::CloseBracket);
	TokenArray.Add(EUDNToken::OpenParenthesis);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::CloseParenthesis);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::Link));
	
	TokenArray.Empty();
	TokenArray.Add(EUDNToken::OpenBracket);
	TokenArray.Add(EUDNToken::Bang);
	TokenArray.Add(EUDNToken::OpenBracket);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::CloseBracket);
	TokenArray.Add(EUDNToken::OpenParenthesis);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::CloseParenthesis);
	TokenArray.Add(EUDNToken::CloseBracket);
	TokenArray.Add(EUDNToken::OpenParenthesis);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::CloseParenthesis);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::ImageLink));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::Bang);
	TokenArray.Add(EUDNToken::OpenBracket);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::CloseBracket);
	TokenArray.Add(EUDNToken::OpenParenthesis);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::CloseParenthesis);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::Image));
	
	TokenArray.Empty();
	TokenArray.Add(EUDNToken::OpenBracket);
	TokenArray.Add(EUDNToken::Excerpt);
	TokenArray.Add(EUDNToken::Colon);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::CloseBracket);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::ExcerptOpen));
	
	TokenArray.Empty();
	TokenArray.Add(EUDNToken::OpenBracket);
	TokenArray.Add(EUDNToken::Slash);
	TokenArray.Add(EUDNToken::Excerpt);
	TokenArray.Add(EUDNToken::Colon);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::CloseBracket);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::ExcerptClose));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::MetadataAvailability);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::MetadataAvailability, true));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::MetadataTitle);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::MetadataTitle, true));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::MetadataCrumbs);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::MetadataCrumbs, true));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::MetadataDescription);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::MetadataDescription, true));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::MetadataBaseUrl);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::MetadataBaseUrl, true));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::OpenBracket);
	TokenArray.Add(EUDNToken::Variable);
	TokenArray.Add(EUDNToken::Colon);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::CloseBracket);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::OpenBracket);
	TokenArray.Add(EUDNToken::Variable);
	TokenArray.Add(EUDNToken::CloseBracket);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::Variable));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::OpenBracket);
	TokenArray.Add(EUDNToken::Variable);
	TokenArray.Add(EUDNToken::Colon);
	TokenArray.Add(EUDNToken::Content);
	TokenArray.Add(EUDNToken::CloseBracket);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::VariableOpen));

	TokenArray.Empty();
	TokenArray.Add(EUDNToken::OpenBracket);
	TokenArray.Add(EUDNToken::Slash);
	TokenArray.Add(EUDNToken::Variable);
	TokenArray.Add(EUDNToken::CloseBracket);
	LineLibrary.Add(FTokenConfiguration(TokenArray, FUDNLine::VariableClose));
}

FUDNParser::~FUDNParser()
{
	if ( FModuleManager::Get().IsModuleLoaded("MessageLog") )
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.UnregisterLogListing(UDNParseErrorLog);
	}
}

bool FUDNParser::LoadLink( const FString& Link, TArray<FString>& ContentLines )
{
	FMessageLog UDNParserLog(UDNParseErrorLog);
	
	if (!FPaths::FileExists(Link))
	{
		return false;
	}

	TArray<uint8> Buffer;
	bool bLoadSuccess = FFileHelper::LoadFileToArray(Buffer, *Link);
	if ( bLoadSuccess )
	{
		FString Result;
		FFileHelper::BufferToString( Result, Buffer.GetData(), Buffer.Num() );

		// Now read it
		// init traveling pointer
		TCHAR* Ptr = Result.GetCharArray().GetData();

		// iterate over the lines until complete
		bool bIsDone = false;
		while (!bIsDone)
		{
			// Store the location of the first character of this line
			TCHAR* Start = Ptr;

			// Advance the char pointer until we hit a newline character
			while (*Ptr && *Ptr!='\r' && *Ptr!='\n')
			{
				Ptr++;
			}

			// If this is the end of the file, we're done
			if (*Ptr == 0)
			{
				bIsDone = 1;
			}
			// Handle different line endings. If \r\n then NULL and advance 2, otherwise NULL and advance 1
			// This handles \r, \n, or \r\n
			else if ( *Ptr=='\r' && *(Ptr+1)=='\n' )
			{
				// This was \r\n. Terminate the current line, and advance the pointer forward 2 characters in the stream
				*Ptr++ = 0;
				*Ptr++ = 0;
			}
			else
			{
				// Terminate the current line, and advance the pointer to the next character in the stream
				*Ptr++ = 0;
			}

			ContentLines.Add(Start);
		}
	}
	else
	{
		UDNParserLog.Error(FText::Format(LOCTEXT("LoadingError", "Loading document '{0}' failed."), FText::FromString(Link)));
	}

	if ( !bLoadSuccess && GetDefault<UEditorPerProjectUserSettings>()->bDisplayDocumentationLink )
	{
		UDNParserLog.Open();
	}

	return bLoadSuccess;
}

bool FUDNParser::Parse(const FString& Link, TArray<FExcerpt>& OutExcerpts, FUDNPageMetadata& OutMetadata)
{
	FMessageLog UDNParserLog(UDNParseErrorLog);

	// Call LoadLink for all content folders, and amalgamate the excerpts from all matching UDN files into a single list.
	const TArray<FString> SourcePaths = IDocumentation::Get()->GetSourcePaths();
	for (const FString& SourcePath : SourcePaths)
	{
		FString DocFilePath = FDocumentationLink::ToSourcePath(Link, SourcePath);
		TArray<FString> ContentLines;
		if (LoadLink(DocFilePath, ContentLines))
		{
			TArray<FExcerpt> TempExcerpts;
			bool bParseSuccess = ParseSymbols(DocFilePath, ContentLines, FPaths::GetPath(DocFilePath), TempExcerpts, OutMetadata);

			if (bParseSuccess)
			{
				OutExcerpts.Append(TempExcerpts);
			}
			else
			{
				if (GetDefault<UEditorPerProjectUserSettings>()->bDisplayDocumentationLink)
				{
					UDNParserLog.Open();
				}

				UDNParserLog.Error(FText::Format(LOCTEXT("GeneralParsingError", "Parsing document '{0}' failed."), FText::FromString(DocFilePath)));
			}
		}
	}
	return true;
}

bool FUDNParser::GetExcerptContent( const FString& Link, FExcerpt& Excerpt )
{
	FMessageLog UDNParserLog(UDNParseErrorLog);

	FString SourcePath = FDocumentationLink::ToSourcePath(Link);
	if (!Excerpt.SourcePath.IsEmpty())
	{
		SourcePath = Excerpt.SourcePath;
	}
	TArray<FString> ContentLines;

	if ( LoadLink(SourcePath, ContentLines ) )
	{
		Excerpt.Content = GenerateExcerptContent(SourcePath, Excerpt, ContentLines, Excerpt.LineNumber );
		return true;
	}
	else
	{
		if ( GetDefault<UEditorPerProjectUserSettings>()->bDisplayDocumentationLink )
		{
			UDNParserLog.Open();
		}

		UDNParserLog.Error(FText::Format(LOCTEXT("GeneralExcerptError", "Generating a Widget for document '{0}' Excerpt '{1}' failed."), FText::FromString(SourcePath), FText::FromString(Excerpt.Name) ));
	}

	return false;
}

void FUDNParser::SetWrapAt( TAttribute<float> InWrapAt )
{
	WrapAt = InWrapAt;
}

int32 FUDNParser::FTokenConfiguration::CalculatedExpectedContentStrings()
{
	int32 ExpectedContentStrings = 0;
	for (int32 i = 0; i < TokensAccepted.Num(); ++i)
	{
		if (TokensAccepted[i] == EUDNToken::Content) {++ExpectedContentStrings;}
	}
	return ExpectedContentStrings;
}

TSharedPtr<FSlateDynamicImageBrush> FUDNParser::GetDynamicBrushFromImagePath(FString Filename)
{
	FName BrushName( *Filename );

	if (FPaths::GetExtension(Filename) == TEXT("png"))
	{
		FArchive* ImageArchive = IFileManager::Get().CreateFileReader(*Filename);
		if (ImageArchive && FSlateApplication::IsInitialized())
		{
			if (FSlateRenderer* Renderer = FSlateApplicationBase::Get().GetRenderer())
			{
				TSharedPtr<FSlateDynamicImageBrush> AlreadyExistingImageBrush;
				for (int32 i = 0; i < DynamicBrushesUsed.Num(); ++i)
				{
					if (DynamicBrushesUsed[i]->GetResourceName() == BrushName) { AlreadyExistingImageBrush = DynamicBrushesUsed[i]; break; }
				}

				if (AlreadyExistingImageBrush.IsValid())
				{
					return AlreadyExistingImageBrush;
				}
				else
				{
					FIntPoint Size = Renderer->GenerateDynamicImageResource(BrushName);
					return MakeShareable(new FSlateDynamicImageBrush(BrushName, FVector2D(Size.X, Size.Y)));
				}
			}
		}
	}

	return TSharedPtr<FSlateDynamicImageBrush>();
}

FString FUDNParser::ConvertSymbolIntoAString(const FUDNToken& Token)
{
	if (Token.TokenType == EUDNToken::Content)
	{
		return Token.Content;
	}

	for (int32 i = 0; i < TokenLibrary.Num(); ++i)
	{
		auto& LibraryToken = TokenLibrary[i];
		if (LibraryToken.TokenType == Token.TokenType)
		{
			return LibraryToken.ParseText;
		}
	}
	return FString();
}

FString FUDNParser::ConvertSymbolsIntoAString(const TArray<FUDNToken>& TokenList, int32 StartingAfterIndex)
{
	bool bIsInVariableSubstitution = false;
	FString Output;
	for (int32 i = StartingAfterIndex; i < TokenList.Num(); ++i)
	{
		auto& Token = TokenList[i];

		if(Token.TokenType == EUDNToken::Percentage)
		{
			bIsInVariableSubstitution = !bIsInVariableSubstitution;
		}

		if(!bIsInVariableSubstitution && Token.TokenType != EUDNToken::Percentage)
		{
			Output += ConvertSymbolIntoAString(Token);
		}
	}
	return Output;
}

bool FUDNParser::ParseLineIntoSymbols(int32 LineNumber, const FString& Line, TArray<FUDNToken>& SymbolList)
{
	if (!Line.IsEmpty())
	{
		FString ChoppedLine;

		bool bFoundSymbol = false;
		for (int32 i = 0; i < TokenLibrary.Num(); ++i)
		{
			auto& Symbol = TokenLibrary[i];
			FString TrimmedLine = Line;
			TrimmedLine.TrimStartInline();
			if (TrimmedLine.StartsWith(Symbol.ParseText))
			{
				ChoppedLine = TrimmedLine.RightChop(Symbol.ParseText.Len());

				SymbolList.Add(FUDNToken(Symbol.TokenType));

				bFoundSymbol = true;
				break;
			}
		}
		
		if (!bFoundSymbol)
		{
			struct Local
			{
				static bool CharIsValid(const TCHAR& Char)
				{
					return 
						Char != LITERAL(TCHAR, '[') &&
						Char != LITERAL(TCHAR, ']') &&
						Char != LITERAL(TCHAR, '(') &&
						Char != LITERAL(TCHAR, ')') &&
						Char != LITERAL(TCHAR, '%') &&
						Char != LITERAL(TCHAR, '*');
				}

				static bool FirstCharIsValid(const TCHAR& Char)
				{
					return 
						Char != LITERAL(TCHAR, '[') &&
						Char != LITERAL(TCHAR, ']') &&
						Char != LITERAL(TCHAR, '(') &&
						Char != LITERAL(TCHAR, ')') &&
						Char != LITERAL(TCHAR, '!') &&
						Char != LITERAL(TCHAR, ':') &&
						Char != LITERAL(TCHAR, '/') &&
						Char != LITERAL(TCHAR, '%') &&
						Char != LITERAL(TCHAR, '*');
				}
			};

			int32 CharIdx = 0;
			for (; CharIdx < Line.Len(); ++CharIdx)
			{
				auto Char = Line[CharIdx];
				bool bIsContentChar = CharIdx == 0 ? Local::FirstCharIsValid(Char) : Local::CharIsValid(Char);

				if ( !bIsContentChar && CharIdx != 0 )
				{
					FString LeftString = Line.Left(CharIdx);
					ChoppedLine = Line.RightChop(CharIdx);

					SymbolList.Add(FUDNToken(EUDNToken::Content, LeftString));

					bFoundSymbol = true;
					break;
				}
			}

			// Indicates that we went to the end of the line, so the entire thing is a symbol
			if (CharIdx == Line.Len())
			{
				ChoppedLine = FString();
				SymbolList.Add(FUDNToken(EUDNToken::Content, Line));
				bFoundSymbol = true;
			}
		}

		if (!bFoundSymbol)
		{
			// Indicates that we found an unknown token, error
			FMessageLog UDNParserLog(UDNParseErrorLog);
			UDNParserLog.Error(FText::Format(LOCTEXT("TokenParseError", "Line {0}: Token '{1}' could not be parsed properly."), FText::AsNumber(LineNumber), FText::FromString(Line)));

			if ( GetDefault<UEditorPerProjectUserSettings>()->bDisplayDocumentationLink )
			{
				UDNParserLog.Open();
			}

			return false;
		}
		else
		{
			return ParseLineIntoSymbols(LineNumber, ChoppedLine, SymbolList);
		}
	}

	// Line is out of characters
	return true;
}

FUDNLine FUDNParser::ParseLineIntoUDNContent(int32 LineNumber, const FString& Line)
{
	FMessageLog UDNParserLog(UDNParseErrorLog);

	FString TrimmedLine = Line;
	TrimmedLine.TrimStartInline();

	FUDNLine OutputLine;

	TArray<FUDNToken> SymbolList;
	bool bSuccessful = ParseLineIntoSymbols(LineNumber, TrimmedLine, SymbolList);

	if (bSuccessful)
	{
		if (SymbolList.Num() > 0)
		{
			bool bLineWasMatched = false;
			for (int32 i = 0; i < LineLibrary.Num() && !bLineWasMatched; ++i)
			{
				auto& LineConfig = LineLibrary[i];

				TArray<FString> Contents;
				FString CurrentContentString;

				bool bMatch = true;
				bool bInVariableSubstitution = false;

				int32 SymbolIdx = 0;
				for (int32 TokenIdx = 0; bMatch && TokenIdx < LineConfig.TokensAccepted.Num(); ++TokenIdx)
				{
					EUDNToken::Type& Token = LineConfig.TokensAccepted[TokenIdx];
					if (SymbolIdx < SymbolList.Num())
					{
						FUDNToken& Symbol = SymbolList[SymbolIdx];
						if(bInVariableSubstitution && Symbol.TokenType != EUDNToken::Percentage)
						{
							++SymbolIdx;
						}
						else if (Symbol.TokenType == EUDNToken::Percentage)
						{
							bInVariableSubstitution = !bInVariableSubstitution;
							++SymbolIdx;
						}
						else
						{
							if (Token == EUDNToken::Content)
							{
								check(TokenIdx + 1 < LineConfig.TokensAccepted.Num() && LineConfig.TokensAccepted[TokenIdx+1] != EUDNToken::Content);
								EUDNToken::Type& NextToken = LineConfig.TokensAccepted[TokenIdx+1];

								if (Symbol.TokenType == NextToken)
								{
									Contents.Add(CurrentContentString);
									CurrentContentString.Empty();
								}
								else
								{
									CurrentContentString += ConvertSymbolIntoAString(Symbol);
									++SymbolIdx;
									--TokenIdx;
								}
							}
							else
							{
								if (Symbol.TokenType != Token)
								{
									bMatch = false;
								}
								++SymbolIdx;
							}
						}
					}
					else
					{
						if(bInVariableSubstitution)
						{
							UDNParserLog.Error(FText::Format(LOCTEXT("VariableSubstitutionError", "Line {0}: Line '{1}' variable substitution was not terminated"), FText::AsNumber(LineNumber), FText::FromString(Line)));
						}

						if (Token != EUDNToken::Content)
						{
							bMatch = false;
						}
					}
				}

				if (bMatch && (SymbolIdx == SymbolList.Num() || LineConfig.bAcceptTrailingSymbolDumpAsContent))
				{
					if (LineConfig.CalculatedExpectedContentStrings() == Contents.Num())
					{
						OutputLine.ContentType = LineConfig.OutputLineType;
						for (const FString& Content : Contents)
						{
							OutputLine.AdditionalContent.Add(Content);
						}
						if (LineConfig.bAcceptTrailingSymbolDumpAsContent)
						{
							OutputLine.AdditionalContent.Add(ConvertSymbolsIntoAString(SymbolList, SymbolIdx).TrimStart());
						}
					}
					else
					{
						if ( GetDefault<UEditorPerProjectUserSettings>()->bDisplayDocumentationLink )
						{
							UDNParserLog.Open();
						}

						UDNParserLog.Error(FText::Format(LOCTEXT("LineConvertError", "Line {0}: Line '{1}' could not converted into a Slate widget."), FText::AsNumber(LineNumber), FText::FromString(Line)));
					}
					check(!bLineWasMatched);
					bLineWasMatched = true;
				}
			}

			if (!bLineWasMatched)
			{
				OutputLine.ContentType = FUDNLine::Content;
				OutputLine.AdditionalContent.Add(ConvertSymbolsIntoAString(SymbolList));
			}
		}
		else
		{
			// empty line
			OutputLine.ContentType = FUDNLine::Whitespace;
		}
	}
	else
	{
		if ( GetDefault<UEditorPerProjectUserSettings>()->bDisplayDocumentationLink )
		{
			UDNParserLog.Open();
		}

		UDNParserLog.Error(FText::Format(LOCTEXT("LineParseError", "Line {0}: Line '{1}' could not be parsed into symbols properly."), FText::AsNumber(LineNumber), FText::FromString(Line)));
	}

	return OutputLine;
}

void FUDNParser::AppendExcerpt(TSharedPtr<SVerticalBox> Box, TSharedRef<SWidget> Content)
{
	Box->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Center)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.WidthOverride(ContentWidth)
		.Padding(FMargin(0,0,0,8.f))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				Content
			]
		]
	];
}

static void AddLineSeperator(FExcerpt& Excerpt)
{
	if(!Excerpt.RichText.IsEmpty())
	{
		Excerpt.RichText += LINE_TERMINATOR;
		Excerpt.RichText += LINE_TERMINATOR;
	}
}

void FUDNParser::AddContentToExcerpt(TSharedPtr<SVerticalBox> Box, const FString& ContentSource, FExcerpt& Excerpt)
{
	if ( !ContentSource.IsEmpty() )
	{
		AppendExcerpt(Box,
			SNew(STextBlock)
			.Text(FText::FromString(ContentSource))
			.TextStyle(FAppStyle::Get(), Style.ContentStyleName)
			.WrapTextAt(WrapAt)
		);

		AddLineSeperator(Excerpt);
		Excerpt.RichText += FString::Printf(TEXT("<TextStyle Style=\"%s\">%s</>"), *Style.ContentStyleName.ToString(), *ContentSource);
	}
}

void FUDNParser::AddCaptionToExcerpt(TSharedPtr<SVerticalBox> Box, const FString& ContentSource, FExcerpt& Excerpt)
{
	if (!ContentSource.IsEmpty())
	{
		AppendExcerpt(Box,
			SNew(SBox)
			.WidthOverride(ContentWidth)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(IndentWidth)
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("ImageCaptionLabel", "Image: {0}"), FText::FromString(ContentSource)))
					.TextStyle(FAppStyle::Get(), Style.ItalicContentStyleName)
					.ColorAndOpacity(FStyleColors::Foreground)
					.WrapTextAt(WrapAt.Get() - IndentWidth)
				]
			]
		);

		AddLineSeperator(Excerpt);
		Excerpt.RichText += FString::Printf(TEXT("<TextStyle Style=\"%s\">%s</>"), *Style.ContentStyleName.ToString(), *ContentSource);
	}
}

void FUDNParser::AddListItemToExcerpt(TSharedPtr<SVerticalBox> Box, const FString& LeftContentSource, const FString& RightContentSource, FExcerpt& Excerpt)
{
	if (!LeftContentSource.IsEmpty() && !RightContentSource.IsEmpty())
	{
		Box->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			[
				SNew(SBox)
				.HAlign(HAlign_Left)
				.WidthOverride(ContentWidth)
				.Padding(FMargin(0, 0, 0, 8.f))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(IndentWidth)
						.Padding(FMargin(16.f, 0, 0, 0))
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Text(FText::FromString(LeftContentSource))
							.TextStyle(FAppStyle::Get(), Style.ContentStyleName)
						]
					]
					+ SHorizontalBox::Slot()
					.FillWidth(100.0f)
					[
						SNew(STextBlock)
						.Text(FText::FromString(RightContentSource))
						.TextStyle(FAppStyle::Get(), Style.ContentStyleName)
						.WrapTextAt(WrapAt.Get() - IndentWidth)
					]
				]
			];

		AddLineSeperator(Excerpt);
		Excerpt.RichText += FString::Printf(TEXT("<TextStyle Style=\"%s\">%s %s</>"), *Style.ContentStyleName.ToString(), *LeftContentSource, *RightContentSource);
	}
}

TSharedRef< SWidget > FUDNParser::GenerateExcerptContent( const FString& InLink, FExcerpt& Excerpt, const TArray<FString>& ContentLines, int32 StartingLineIndex )
{
	FMessageLog UDNParserLog(UDNParseErrorLog);

	const FString FullPath = FPaths::GetPath(InLink);

	bool bCriticalError = false;
	FString VariableName;
	FString CurrentStringContent;
	int32 CurrentNumbering = 1;

	TSharedPtr<SVerticalBox> Box;
	TArray<FString> ExcerptStack;

	for (int32 CurrentLineNumber = StartingLineIndex; CurrentLineNumber < ContentLines.Num(); ++CurrentLineNumber)
	{
		const FString& CurrentLine = ContentLines[ CurrentLineNumber ];
		const FUDNLine& Line = ParseLineIntoUDNContent(CurrentLineNumber, CurrentLine);

		if (Line.ContentType == FUDNLine::ExcerptOpen)
		{
			ExcerptStack.Push(Line.AdditionalContent[0]);
			Box = SNew( SVerticalBox );
		}
		else if (Line.ContentType == FUDNLine::ExcerptClose)
		{
			if (ExcerptStack.Num() == 0 || Line.AdditionalContent[0] != ExcerptStack.Top())
			{
				UDNParserLog.NewPage( FText::FromString( InLink + TEXT(" [") + Excerpt.Name + TEXT("]") ) );
				UDNParserLog.Error(FText::Format(LOCTEXT("ExcerptCloseError", "Line {0}: Excerpt {1} improperly closed."), FText::AsNumber(CurrentLineNumber), FText::FromString(Line.AdditionalContent[0])));
				bCriticalError = true;
				break;
			}

			FString ExcerptName = ExcerptStack.Pop();

			if ( ExcerptStack.Num() == 0 )
			{
				AddContentToExcerpt(Box, CurrentStringContent, Excerpt);
				break;
			}
		}
		else if ( Line.ContentType == FUDNLine::VariableOpen )
		{
			if ( !VariableName.IsEmpty() )
			{
				UDNParserLog.NewPage( FText::FromString( InLink + TEXT(" [") + Excerpt.Name + TEXT("]") ) );
				UDNParserLog.Error(FText::Format(LOCTEXT("VariableOpenError", "Line {0}: Excerpt {1} improperly attempting to define a variable within a variable."), FText::AsNumber(CurrentLineNumber), FText::FromString(Line.AdditionalContent[0])));
				bCriticalError = true;
				break;
			}

			VariableName = Line.AdditionalContent[0];

			if ( VariableName.IsEmpty() )
			{
				UDNParserLog.NewPage( FText::FromString( InLink + TEXT(" [") + Excerpt.Name + TEXT("]") ) );
				UDNParserLog.Error(FText::Format(LOCTEXT("VariableWithOutName", "Line {0}: Excerpt {1} improperly attempted to define a variable with no name."), FText::AsNumber(CurrentLineNumber), FText::FromString(Line.AdditionalContent[0])));
				bCriticalError = true;
				break;
			}
		}
		else if ( Line.ContentType == FUDNLine::VariableClose )
		{
			if ( VariableName.IsEmpty() )
			{
				UDNParserLog.NewPage( FText::FromString( InLink + TEXT(" [") + Excerpt.Name + TEXT("]") ) );
				UDNParserLog.Error(FText::Format(LOCTEXT("VariableCloseError", "Line {0}: Excerpt {1} improperly attempting to close a variable tag it never opened."), FText::AsNumber(CurrentLineNumber), FText::FromString(Line.AdditionalContent[0])));
				bCriticalError = true;
				break;
			}

			VariableName.Empty();
		}
		else if ( Line.ContentType == FUDNLine::Variable )
		{
			if ( Line.AdditionalContent.Num() != 2 )
			{
				UDNParserLog.NewPage( FText::FromString( InLink + TEXT(" [") + Excerpt.Name + TEXT("]") ) );
				UDNParserLog.Error(FText::Format(LOCTEXT("Variable", "Line {0}: Excerpt {1} improperly attempted to define a variable with no name."), FText::AsNumber(CurrentLineNumber), FText::FromString(Line.AdditionalContent[0])));
				bCriticalError = true;
				break;
			}

			VariableName = Line.AdditionalContent[0];

			if ( VariableName.IsEmpty() )
			{
				UDNParserLog.NewPage( FText::FromString( InLink + TEXT(" [") + Excerpt.Name + TEXT("]") ) );
				UDNParserLog.Error(FText::Format(LOCTEXT("VariableWithOutName", "Line {0}: Excerpt {1} improperly attempted to define a variable with no name."), FText::AsNumber(CurrentLineNumber), FText::FromString(Line.AdditionalContent[0])));
				bCriticalError = true;
				break;
			}
		}


		FString ConcatenatedPath;
		TSharedPtr<FSlateDynamicImageBrush> DynamicBrush;

		if (Line.ContentType == FUDNLine::Content && !CurrentStringContent.IsEmpty()) 
		{
			CurrentStringContent += LINE_TERMINATOR;
		}

		// only emit widgets if we are not inside a variable declaration
		if ( VariableName.IsEmpty() )
		{
			switch (Line.ContentType)
			{
			case FUDNLine::Whitespace:
				// Will only apply whitespace for the first empty line
				AddContentToExcerpt(Box, CurrentStringContent, Excerpt);
				CurrentStringContent.Empty();
				break;
			case FUDNLine::Content:
				CurrentStringContent += Line.AdditionalContent[0];
				break;
			case FUDNLine::BoldContent:
				AddContentToExcerpt(Box, CurrentStringContent, Excerpt);
				CurrentStringContent.Empty();

				AppendExcerpt(Box,
					SNew(STextBlock)
					.Text(FText::FromString(Line.AdditionalContent[0]))
					.TextStyle(FAppStyle::Get(), Style.BoldContentStyleName)
					);

				AddLineSeperator(Excerpt);
				Excerpt.RichText += FString::Printf(TEXT("<TextStyle Style=\"%s\">%s</>"), *Style.BoldContentStyleName.ToString(), *Line.AdditionalContent[0]);
				break;
			case FUDNLine::NumberedContent:
				AddContentToExcerpt(Box, CurrentStringContent, Excerpt);
				CurrentStringContent.Empty();
				AddListItemToExcerpt(Box, FString::Printf(TEXT("%i."), CurrentNumbering), Line.AdditionalContent[0], Excerpt);
				++CurrentNumbering;
				break;
			case FUDNLine::BulletContent:
				AddContentToExcerpt(Box, CurrentStringContent, Excerpt);
				CurrentStringContent.Empty();
				AddListItemToExcerpt(Box, TEXT("•"), Line.AdditionalContent[0], Excerpt);
				break;
			case FUDNLine::HorizontalRule:
				AddContentToExcerpt(Box, CurrentStringContent, Excerpt);
				CurrentStringContent.Empty();

				Box->AddSlot()
					.Padding(0.0f, 8.0f, 0.0f, 16.0f)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Documentation.Separator"))
						.Padding(0.0f, 1.0f)
						[
							SNew(SBox)
							.WidthOverride(ContentWidth)
							.HeightOverride(0)
						]
					];

				AddLineSeperator(Excerpt);
				break;
			case FUDNLine::Header1:
				AddContentToExcerpt(Box, CurrentStringContent, Excerpt);
				CurrentStringContent.Empty();

				AppendExcerpt(Box,
					SNew(STextBlock)
					.Text(FText::FromString(Line.AdditionalContent[0]))
					.TextStyle(FAppStyle::Get(), Style.Header1StyleName)
					);

				AddLineSeperator(Excerpt);
				Excerpt.RichText += FString::Printf(TEXT("<TextStyle Style=\"%s\">%s</>"), *Style.Header1StyleName.ToString(), *Line.AdditionalContent[0]);
				break;
			case FUDNLine::Header2:
				AddContentToExcerpt(Box, CurrentStringContent, Excerpt);
				CurrentStringContent.Empty();

				AppendExcerpt(Box,
					SNew(STextBlock)
					.Text(FText::FromString(Line.AdditionalContent[0]))
					.TextStyle(FAppStyle::Get(), Style.Header2StyleName)
					);

				AddLineSeperator(Excerpt);
				Excerpt.RichText += FString::Printf(TEXT("<TextStyle Style=\"%s\">%s</>"), *Style.Header2StyleName.ToString(), *Line.AdditionalContent[0]);
				break;
			case FUDNLine::Header3:
				AddContentToExcerpt(Box, CurrentStringContent, Excerpt);
				CurrentStringContent.Empty();

				AppendExcerpt(Box,
					SNew(STextBlock)
					.Text(FText::FromString(Line.AdditionalContent[0]))
					.TextStyle(FAppStyle::Get(), Style.Header3StyleName)
				);

				AddLineSeperator(Excerpt);
				Excerpt.RichText += FString::Printf(TEXT("<TextStyle Style=\"%s\">%s</>"), *Style.Header3StyleName.ToString(), *Line.AdditionalContent[0]);
				break;
			case FUDNLine::Link:
				AddContentToExcerpt(Box, CurrentStringContent, Excerpt);
				CurrentStringContent.Empty();

				AppendExcerpt(Box,
					SNew(SHyperlink)
					.Text(FText::FromString(Line.AdditionalContent[0]))
					.TextStyle(FAppStyle::Get(), Style.HyperlinkTextStyleName)
					.UnderlineStyle(FAppStyle::Get(), Style.HyperlinkButtonStyleName)
					.OnNavigate( this, &FUDNParser::HandleHyperlinkNavigate, Line.AdditionalContent[1])
					);

				AddLineSeperator(Excerpt);

				if(Line.AdditionalContent[1].Contains(LinkPrefixes::DocLinkSpecifier))
				{
					const FString Link = Line.AdditionalContent[1].RightChop(LinkPrefixes::DocLinkSpecifier.Len());
					Excerpt.RichText += FString::Printf(TEXT("<a id=\"udn\" href=\"%s\" style=\"%s\">%s</>"), *Link, *Style.HyperlinkStyleName.ToString(), *Line.AdditionalContent[0]);
				}
				else if(Line.AdditionalContent[1].Contains(LinkPrefixes::AssetLinkSpecifier))
				{
					const FString Link = Line.AdditionalContent[1].RightChop(LinkPrefixes::AssetLinkSpecifier.Len());
					Excerpt.RichText += FString::Printf(TEXT("<a id=\"asset\" href=\"%s\" style=\"%s\">%s</>"), *Link, *Style.HyperlinkStyleName.ToString(), *Line.AdditionalContent[0]);
				}
				else if(Line.AdditionalContent[1].Contains(LinkPrefixes::CodeLinkSpecifier))
				{
					const FString Link = Line.AdditionalContent[1].RightChop(LinkPrefixes::CodeLinkSpecifier.Len());
					Excerpt.RichText += FString::Printf(TEXT("<a id=\"code\" href=\"%s\" style=\"%s\">%s</>"), *Link, *Style.HyperlinkStyleName.ToString(), *Line.AdditionalContent[0]);
				}
				else
				{
					Excerpt.RichText += FString::Printf(TEXT("<a id=\"browser\" href=\"%s\" style=\"%s\">%s</>"), *InLink, *Style.HyperlinkStyleName.ToString(), *Line.AdditionalContent[0]);
				}
				break;
			case FUDNLine::Image:
				ConcatenatedPath = FullPath / TEXT("Images") / Line.AdditionalContent[1];
				DynamicBrush = GetDynamicBrushFromImagePath(ConcatenatedPath);
				if (DynamicBrush.IsValid())
				{
					AddContentToExcerpt(Box, CurrentStringContent, Excerpt);
					CurrentStringContent.Empty();

					AppendExcerpt(Box,
						SNew( SImage )
						.Image(DynamicBrush.Get())
						.ToolTipText(FText::FromString(Line.AdditionalContent[0]))
						.DesiredSizeOverride(GetImageDesiredSize(DynamicBrush, ContentWidth.Get().Get()))
						);

					DynamicBrushesUsed.AddUnique(DynamicBrush);

					if (!Line.AdditionalContent[0].IsEmpty())
					{
						AddCaptionToExcerpt(Box, Line.AdditionalContent[0], Excerpt);
					}
				}

				AddLineSeperator(Excerpt);
				Excerpt.RichText += FString::Printf(TEXT("<img src=\"%s\"></>"), *ConcatenatedPath);
				break;
			case FUDNLine::ImageLink:
				ConcatenatedPath = FullPath / TEXT("Images") / Line.AdditionalContent[1];
				DynamicBrush = GetDynamicBrushFromImagePath(ConcatenatedPath);
				if (DynamicBrush.IsValid())
				{
					AddContentToExcerpt(Box, CurrentStringContent, Excerpt);
					CurrentStringContent.Empty();

					AppendExcerpt(Box,
						SNew(SButton)
						.ContentPadding(0)
						.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
						.OnClicked( FOnClicked::CreateSP( this, &FUDNParser::OnImageLinkClicked, Line.AdditionalContent[2] ) )
						[
							SNew( SImage )
							.Image(DynamicBrush.Get())
							.ToolTipText(FText::FromString(Line.AdditionalContent[0]))
						]
					);

					DynamicBrushesUsed.AddUnique(DynamicBrush);
				}

				AddLineSeperator(Excerpt);
				Excerpt.RichText += FString::Printf(TEXT("<img src=\"%s\" href=\"%s\"></>"), *ConcatenatedPath, *Line.AdditionalContent[2]);
				break;
			default: break;
			}
		}
	}

	if ( ExcerptStack.Num() > 0 )
	{
		if ( !bCriticalError )
		{
			UDNParserLog.NewPage( FText::FromString( InLink + TEXT(" [") + Excerpt.Name + TEXT("]") ) );
		}

		for (int32 i = 0; i < ExcerptStack.Num(); ++i)
		{
			UDNParserLog.Error(FText::Format(LOCTEXT("ExcerptMismatchError", "Excerpt {0} was never closed."), FText::FromString(ExcerptStack.Top())));
		}
		bCriticalError = true;
	}

	if ( bCriticalError && GetDefault<UEditorPerProjectUserSettings>()->bDisplayDocumentationLink )
	{
		UDNParserLog.Open();
	}

	if ( bCriticalError )
	{
		return SNew( STextBlock ).Text( LOCTEXT("ExcerptContentLoadingError", "Excerpt {0} could not be loaded.  :(") );
	}

	return Box.ToSharedRef();
}

bool FUDNParser::ParseSymbols(const FString& Link, const TArray<FString>& ContentLines, const FString& FullPath, TArray<FExcerpt>& OutExcerpts, FUDNPageMetadata& OutMetadata)
{
	FMessageLog UDNParserLog(UDNParseErrorLog);
	
	bool bCriticalError = false;
	FString CurrentStringContent;
	TArray<FString> ExcerptStack;
	int32 ExcerptStartingLineNumber = 0;

	FString VariableName;
	FString VariableValue;
	TMap< FString, FString > Variables;
	for (int32 CurrentLineNumber = 0; CurrentLineNumber < ContentLines.Num(); ++CurrentLineNumber)
	{
		const FString& CurrentLine = ContentLines[ CurrentLineNumber ];

		const FUDNLine& Line = ParseLineIntoUDNContent(CurrentLineNumber, CurrentLine);
		
		bool bIsReadingContent = ExcerptStack.Num() > 0;
		
		if (Line.ContentType == FUDNLine::ExcerptOpen)
		{
			if ( ExcerptStack.Num() == 0 )
			{
				ExcerptStartingLineNumber = CurrentLineNumber;
			}

			ExcerptStack.Push(Line.AdditionalContent[0]);
		}
		else if (Line.ContentType == FUDNLine::ExcerptClose)
		{
			if (ExcerptStack.Num() == 0 || Line.AdditionalContent[0] != ExcerptStack.Top())
			{
				UDNParserLog.NewPage( FText::FromString( Link ) );
				UDNParserLog.Error(FText::Format(LOCTEXT("ExcerptCloseError", "Line {0}: Excerpt {1} improperly closed."), FText::AsNumber(CurrentLineNumber), FText::FromString(Line.AdditionalContent[0])));
				bCriticalError = true;
				break;
			}

			FString ExcerptName = ExcerptStack.Pop();
			
			if (ExcerptStack.Num() == 0)
			{
				if (!OutMetadata.BaseUrl.IsEmpty() && !Variables.Contains("BaseUrl"))
				{
					Variables.Add("BaseUrl", OutMetadata.BaseUrl);
				}
				if (Variables.Contains("ExcerptAlias") && !Variables.Find("ExcerptAlias")->IsEmpty())
				{
					OutMetadata.ExcerptAliases.Add(ExcerptName, *Variables.Find("ExcerptAlias"));
				}
				OutExcerpts.Add(FExcerpt(ExcerptName, NULL, Variables, ExcerptStartingLineNumber, Link));
				OutMetadata.ExcerptNames.Add( ExcerptName );
				Variables.Empty();
				ExcerptStartingLineNumber = 0;
			}
		}
		else if ( Line.ContentType == FUDNLine::VariableOpen )
		{
			if ( !VariableName.IsEmpty() )
			{
				UDNParserLog.NewPage( FText::FromString( Link ) );
				UDNParserLog.Error(FText::Format(LOCTEXT("VariableOpenError", "Line {0}: Excerpt {1} improperly attempting to define a variable within a variable."), FText::AsNumber(CurrentLineNumber), FText::FromString(Line.AdditionalContent[0])));
				bCriticalError = true;
				break;
			}

			VariableName = Line.AdditionalContent[0];

			if ( VariableName.IsEmpty() )
			{
				UDNParserLog.NewPage( FText::FromString( Link ) );
				UDNParserLog.Error(FText::Format(LOCTEXT("VariableWithOutName", "Line {0}: Excerpt {1} improperly attempted to define a variable with no name."), FText::AsNumber(CurrentLineNumber), FText::FromString(Line.AdditionalContent[0])));
				bCriticalError = true;
				break;
			}
		}
		else if ( Line.ContentType == FUDNLine::VariableClose )
		{
			if ( VariableName.IsEmpty() )
			{
				UDNParserLog.NewPage( FText::FromString( Link ) );
				UDNParserLog.Error(FText::Format(LOCTEXT("VariableCloseError", "Line {0}: Excerpt {1} improperly attempting to close a variable tag it never opened."), FText::AsNumber(CurrentLineNumber), FText::FromString(Line.AdditionalContent[0])));
				bCriticalError = true;
				break;
			}

			Variables.Add( VariableName, VariableValue.TrimStartAndEnd());

			VariableName.Empty();
			VariableValue.Empty();
		}
		else if ( Line.ContentType == FUDNLine::Variable )
		{
			if ( Line.AdditionalContent.Num() != 2 )
			{
				UDNParserLog.NewPage( FText::FromString( Link ) );
				UDNParserLog.Error(FText::Format(LOCTEXT("Variable", "Line {0}: Excerpt {1} improperly attempted to define a variable with no name."), FText::AsNumber(CurrentLineNumber), FText::FromString(Line.AdditionalContent[0])));
				bCriticalError = true;
				break;
			}

			VariableName = Line.AdditionalContent[0];
			VariableValue = Line.AdditionalContent[1];

			if ( VariableName.IsEmpty() )
			{
				UDNParserLog.NewPage( FText::FromString( Link ) );
				UDNParserLog.Error(FText::Format(LOCTEXT("VariableWithOutName", "Line {0}: Excerpt {1} improperly attempted to define a variable with no name."), FText::AsNumber(CurrentLineNumber), FText::FromString(Line.AdditionalContent[0])));
				bCriticalError = true;
				break;
			}

			Variables.Add( VariableName, VariableValue.TrimStartAndEnd());

			VariableName.Empty();
			VariableValue.Empty();
		}

		if (!bIsReadingContent)
		{
			switch (Line.ContentType)
			{
			case FUDNLine::MetadataAvailability: OutMetadata.Availability = Line.AdditionalContent[0]; break;
			case FUDNLine::MetadataTitle: OutMetadata.Title = FText::FromString(Line.AdditionalContent[0]); break;
			case FUDNLine::MetadataCrumbs: OutMetadata.Crumbs = FText::FromString(Line.AdditionalContent[0]); break;
			case FUDNLine::MetadataDescription: OutMetadata.Description = FText::FromString(Line.AdditionalContent[0]); break;
			case FUDNLine::MetadataBaseUrl: OutMetadata.BaseUrl = Line.AdditionalContent[0]; break;
			}
		}
		else
		{
			switch (Line.ContentType)
			{
			case FUDNLine::Content:
			case FUDNLine::NumberedContent:
			case FUDNLine::BulletContent:
			case FUDNLine::Header1:
			case FUDNLine::Header2:
			case FUDNLine::Image:
			case FUDNLine::Link:
			case FUDNLine::ImageLink:
				{
					if ( !VariableName.IsEmpty() )
					{
						if (!VariableValue.IsEmpty())
						{
							VariableValue += '\n';
						}
						VariableValue += Line.AdditionalContent[0];
					}
				}
				break;
			case FUDNLine::Whitespace:
				{
					if (!VariableName.IsEmpty())
					{
						if (!VariableValue.IsEmpty())
						{
							VariableValue += '\n';
						}
					}
				}
			}
		}
	}

	if ( ExcerptStack.Num() > 0 )
	{
		if ( !bCriticalError )
		{
			UDNParserLog.NewPage( FText::FromString( Link ) );
		}

		for (int32 i = 0; i < ExcerptStack.Num(); ++i)
		{
			UDNParserLog.Error(FText::Format(LOCTEXT("ExcerptMismatchError", "Excerpt {0} was never closed."), FText::FromString(ExcerptStack.Top())));
		}
		bCriticalError = true;
	}

	return !bCriticalError;
}

FReply FUDNParser::OnImageLinkClicked( FString AdditionalContent )
{
	NavigateToLink( AdditionalContent );

	return FReply::Handled();
}

void FUDNParser::HandleHyperlinkNavigate( FString AdditionalContent )
{
	NavigateToLink( AdditionalContent );
}

void FUDNParser::NavigateToLink( FString AdditionalContent )
{
	static const FString DocLinkSpecifier( TEXT( "DOCLINK:" ) );
	static const FString HttpLinkSPecifier( TEXT( "http://" ) );
	static const FString HttpsLinkSPecifier( TEXT( "https://" ) );

	static const FString CodeLinkSpecifier(TEXT("CODELINK:"));
	static const FString AssetLinkSpecifier(TEXT("ASSETLINK:"));

	if (AdditionalContent.StartsWith(DocLinkSpecifier))
	{
		// external link to documentation
		FString DocLink = AdditionalContent.RightChop(DocLinkSpecifier.Len());
		IDocumentation::Get()->Open(DocLink, FDocumentationSourceInfo(TEXT("udn_parser")));
	}
	else if ( AdditionalContent.StartsWith( HttpLinkSPecifier ) || AdditionalContent.StartsWith( HttpsLinkSPecifier ) )
	{
		// external link
		FPlatformProcess::LaunchURL( *AdditionalContent, nullptr, nullptr);
	}
	else if (AdditionalContent.StartsWith(CodeLinkSpecifier))
	{		
		FString InternalLink = AdditionalContent.RightChop(CodeLinkSpecifier.Len()); 		
		ParseCodeLink(InternalLink);
	}	
	else if (AdditionalContent.StartsWith(AssetLinkSpecifier))
	{
		FString InternalLink = AdditionalContent.RightChop(AssetLinkSpecifier.Len());
		ParseAssetLink(InternalLink);
	}
	else
	{
		// internal link
		Configuration->OnNavigate.ExecuteIfBound( AdditionalContent );
	}
}

bool FUDNParser::ParseCodeLink(FString &InternalLink)
{
	// Tokens used by the code parsing. Details in the parse section	
	static const FString ProjectSpecifier(TEXT("[PROJECT]"));
	static const FString ProjectRoot(TEXT("[PROJECT]/Source/[PROJECT]/"));
	static const FString ProjectSuffix(TEXT(".uproject"));

	bool bLinkParsedOK = false;	
	FString Path;
	int32 Line = 0;
	int32 Col = 0;

	TArray<FString> Tokens;
	InternalLink.ParseIntoArray(Tokens, TEXT(","), 0);
	int32 TokenStringsCount = Tokens.Num();
	if (TokenStringsCount > 0)
	{
		Path = Tokens[0];
	}
	if (TokenStringsCount > 1)
	{
		TTypeFromString<int32>::FromString(Line, *Tokens[1]);
	}
	if (TokenStringsCount > 2)
	{
		TTypeFromString<int32>::FromString(Col, *Tokens[2]);
	}

	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();

	// If we specified generic project specified as the project name try to replace the name with the name of this project
	if (InternalLink.Contains(ProjectSpecifier) == true)
	{
		FString ProjectName = TEXT("Marble");
		// Try to extract the name of the project
		FString ProjectPath = FPaths::GetProjectFilePath();
		if (ProjectPath.EndsWith(ProjectSuffix))
		{
			int32 ProjectPathEndIndex;
			if (ProjectPath.FindLastChar(TEXT('/'), ProjectPathEndIndex) == true)
			{
				ProjectName = ProjectPath.RightChop(ProjectPathEndIndex + 1);
				ProjectName.RemoveFromEnd(*ProjectSuffix);
			}
		}
		// Replace the root path with the name of this project
		FString RebuiltPath = ProjectRoot + Path;
		RebuiltPath.ReplaceInline(*ProjectSpecifier, *ProjectName);
		Path = RebuiltPath;
	}

	// Finally create the complete path - project name and all
	return SourceCodeAccessor.OpenFileAtLine(FPaths::RootDir() + Path, Line, Col);
}

bool FUDNParser::ParseAssetLink(FString &InternalLink)
{
	TArray<FString> Token;
	InternalLink.ParseIntoArray(Token, TEXT(","), 0);
	
	if (Token.Num() >= 2)
	{
		FString Action = Token[0];
		FString AssetName = Token[1];

		UObject* RequiredObject = FindFirstObject<UObject>(*AssetName, EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous);
		if (RequiredObject != nullptr)
		{
			if (Action == TEXT("EDIT"))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(RequiredObject);
			}
			else
			{
				FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				TArray<UObject*> AssetToBrowse;
				AssetToBrowse.Add(RequiredObject);
				ContentBrowserModule.Get().SyncBrowserToAssets(AssetToBrowse);
			}
		}
	}
	
	return false;
}

TOptional<FVector2D> FUDNParser::GetImageDesiredSize(TSharedPtr<FSlateDynamicImageBrush> DynamicBrush, float MaxWidth)
{
	if (MaxWidth && DynamicBrush->GetImageSize().X > MaxWidth)
	{
		float ratio = MaxWidth / DynamicBrush->GetImageSize().X;
		return FVector2D(MaxWidth, ratio * DynamicBrush->GetImageSize().Y);
	}
	return FVector2D(DynamicBrush->GetImageSize());
}

#undef LOCTEXT_NAMESPACE
