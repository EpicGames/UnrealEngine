// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDocumentation.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateStructs.h"

class SVerticalBox;
class SWidget;
struct FExcerpt;
struct FSlateDynamicImageBrush;

/** Stores all the metadata that a UDN page can have */
struct FUDNPageMetadata
{
	FUDNPageMetadata()
		: Availability()
		, Title()
		, Crumbs()
		, Description()
		, ExcerptNames()
		, ExcerptAliases()
		, BaseUrl()
	{}

	FString Availability;
	FText Title;
	FText Crumbs;
	FText Description;
	TSet< FString > ExcerptNames;
	TMap< FString, FString > ExcerptAliases;
	FString BaseUrl;
};

/** Represents a single UDN Markdown token */
namespace EUDNToken
{
	enum Type
	{
		Content,
		Pound,
		OpenBracket,
		CloseBracket,
		OpenParenthesis,
		CloseParenthesis,
		Numbering,
		Bang,
		Excerpt,
		Variable,
		Colon,
		Slash,
		Dash,
		MetadataAvailability,
		MetadataTitle,
		MetadataCrumbs,
		MetadataDescription,
		MetadataBaseUrl,
		Percentage,
		Asterisk
	};
}

/** A token, which can also have content */
struct FUDNToken
{
	FUDNToken(EUDNToken::Type InTokenType, const FString& InContent = FString())
		: TokenType(InTokenType)
		, Content(InContent) {}

	EUDNToken::Type TokenType;
	FString Content;
};

/** A UDN line, which, since we parse line by line, will correspond to a single slate widget */
struct FUDNLine
{
public:
	/** The type of line this is */
	enum Type
	{
		Ignored,
		VariableReference,
		Whitespace,
		Content,
		NumberedContent,
		Header1,
		Header2,
		Header3,
		ExcerptOpen,
		ExcerptClose,
		Image,
		Link,
		ImageLink,
		HorizontalRule,
		MetadataAvailability,
		MetadataTitle,
		MetadataCrumbs,
		MetadataDescription,
		MetadataBaseUrl,
		Variable,
		VariableOpen,
		VariableClose,
		BoldContent,
		BulletContent
	};
	Type ContentType;

	/** Optional string/path content that is used by this line */
	TArray<FString> AdditionalContent;

	FUDNLine()
		: ContentType(FUDNLine::Ignored) {}

	FUDNLine(FUDNLine::Type InLineType, const FString& InStringContent = FString(), const FString& InPathContent = FString())
		: ContentType(InLineType) {}
};


/**
 * A Parser for UDN Files, turning them into Slate widgets
 * It only provides a very small subset of what UDN files have to offer:
 *  - Text
 *  - Headers
 *  - Numbering
 *  - Horizontal Rules
 *  - Images (can't be inline)
 *  - Hyperlinks (can't be inline)
 *  - Image Hyperlinks
 *  - Excerpts (currently, it only parses what's in excerpts)
 * Currently, it parses pages into an array based on the excerpts
 */
class FUDNParser : public TSharedFromThis< FUDNParser >
{
public:

	static TSharedRef< FUDNParser > Create( const TSharedPtr< FParserConfiguration >& ParserConfig, const FDocumentationStyle& Style );

public:
	~FUDNParser();

	/**
	 * Parses the UDN page specified by the Path, returning true if successful,
	 * and giving back a list of widgets created by the parsing, split
	 * based on the beginning and ending of excerpts
	 */
	bool Parse(const FString& Link, TArray< FExcerpt >& OutContentSections, FUDNPageMetadata& OutMetadata);

	bool GetExcerptContent( const FString& Link, FExcerpt& Excerpt );

	/** Allows a TAttribute to be set to control excerpt wrapat values from outside the parser. */
	void SetWrapAt( TAttribute<float> InWrapAt );

private:
	/** UI Callbacks for the widgets */
	FReply OnImageLinkClicked(FString Payload);
	
	/** Adds the content text source to the scrollbox */
	void AddContentToExcerpt(TSharedPtr<SVerticalBox> Box, const FString& ContentSource, FExcerpt& Excerpt);
	
	/** Adds the content text source to the scrollbox as a caption for an image */
	void AddCaptionToExcerpt(TSharedPtr<SVerticalBox> Box, const FString& ContentSource, FExcerpt& Excerpt);

	/** Adds the content text sources to the scrollbox in two columns, emulating a list item with hanging indent */
	void AddListItemToExcerpt(TSharedPtr<SVerticalBox> Box, const FString& LeftContentSource, const FString& RightContentSource, FExcerpt& Excerpt);

	/** Gets the dynamic brush for the given filename */
	TSharedPtr<FSlateDynamicImageBrush> GetDynamicBrushFromImagePath(FString Filename);

	/** Helper function which appends a content section to the scrollbox */
	void AppendExcerpt(TSharedPtr<SVerticalBox> Box, TSharedRef<SWidget> Content);
	
	/** Turns a series of symbols (or a single symbol) back to string format */
	FString ConvertSymbolIntoAString(const FUDNToken& Token);
	FString ConvertSymbolsIntoAString(const TArray<FUDNToken>& TokenList, int32 StartingAfterIndex = 0);

	/** Given a line, converts it into UDN tokens, returning true if successful */
	bool ParseLineIntoSymbols(int32 LineNumber, const FString& Line, TArray<FUDNToken>& SymbolList);

	/** Given a line, convert it into an FUDNLine which can be used by Slate */
	FUDNLine ParseLineIntoUDNContent(int32 LineNumber, const FString& Line);

	bool LoadLink( const FString& Link, TArray<FString>& ContentLines );

	TSharedRef< SWidget > GenerateExcerptContent( const FString& Link, FExcerpt& Excerpt, const TArray<FString>& ContentLines, int32 StartingLineIndex );

	bool ParseSymbols( const FString& Link, const TArray<FString>& ContentLines, const FString& FullPath, TArray<FExcerpt>& OutExcerpts, FUDNPageMetadata& OutMetadata );

	void HandleHyperlinkNavigate( FString AdditionalContent );

	void NavigateToLink( FString AdditionalContent );

private:
	/** 
	 * Parses a code link embedded in the doc.
	 * Allows us to specify files in code to link to in one of 2 ways. In both cases the last 2 parameters are line and column.
	 * [Project based link](CODELINK:Private/[PROJECT]File.cpp, 29, 5)
	 * This will attempt to parse the active solution name and replace instances of [PROJECT] within this so
	 * (CODELINK:Private/[PROJECT]Ball.cpp, 29, 5) will in a project called marble will equate to
	 * <UE4ROOT>Marble/Source/Marble/Private/MarbleBall.cpp
	 * [Explicit link](CODELINK:Templates/TP_Rolling/Source/TP_Rolling/Private/TP_Rolling.cpp will equate to
	 * <UE4ROOT>Templates/TP_Rolling/Source/TP_Rolling/Private/TP_Rolling.cpp	 
	 */
	bool ParseCodeLink(FString &InternalLink);
		
	/** 
	 * Parses an asset link embedded in the doc.
	 * Allows us to specify assets to either highlight or edit in the editor
	 * (ASSETLINK:SELECT,MyCharacter)
	 * This will highlight the MyCharacter asset in the content browser
	 * (ASSETLINK:EDIT,MyCharacter)
	 * This will edit the given asset in the appropriate editor window type.
	 */
	bool ParseAssetLink(FString &InternalLink);

	FUDNParser( const TSharedRef< FParserConfiguration >& InConfiguration, const FDocumentationStyle& InStyle );
	void Initialize();

	static TOptional<FVector2D> GetImageDesiredSize(TSharedPtr<FSlateDynamicImageBrush> DynamicBrush, float MaxWidth);

	/** A list of dynamic brushes we are using for the currently loaded tutorial */
	TArray< TSharedPtr< FSlateDynamicImageBrush > > DynamicBrushesUsed;

	/** Configuration details */
	TSharedRef< FParserConfiguration > Configuration;

	/** The styling we apply to generated widgets */
	FDocumentationStyle Style;

	/** A library of which tokens correspond to which series of strings */
	struct FTokenPair
	{
		FTokenPair(FString InParseText, EUDNToken::Type InTokenType)
			: ParseText(InParseText)
			, TokenType(InTokenType) {}

		FString ParseText;
		EUDNToken::Type TokenType;
	};
	TArray<FTokenPair> TokenLibrary;

	/** A library of which series of tokens correspond to which line types */
	struct FTokenConfiguration
	{
		FTokenConfiguration(const TArray<EUDNToken::Type>& InTokenTypes, FUDNLine::Type InOutputLineType, bool bInAcceptTrailingContent = false)
			: TokensAccepted(InTokenTypes)
			, OutputLineType(InOutputLineType)
			, bAcceptTrailingSymbolDumpAsContent(bInAcceptTrailingContent) {}
		
		/** Tallies the total number of content tokens in this config */
		int32 CalculatedExpectedContentStrings();

		TArray<EUDNToken::Type> TokensAccepted;
		FUDNLine::Type OutputLineType;
		bool bAcceptTrailingSymbolDumpAsContent;
	};

	TArray<FTokenConfiguration> LineLibrary;
	/** Documentation text wrapping control attribute */
	TAttribute<float> WrapAt;
	/** Documentation optional width control attribute */
	TAttribute<FOptionalSize> ContentWidth;
};

