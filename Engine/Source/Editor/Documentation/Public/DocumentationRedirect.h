// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "DocumentationUrl.h"

#include "DocumentationRedirect.generated.h"

/** Engine level documentation link redirect */
USTRUCT()
struct FDocumentationRedirect
{
	GENERATED_BODY()

	/**
	 * From Key
	 * 
	 * @note Documentation redirects supports redirection using a UnrealEd URL config key or raw documentation link.
	 *		 This key will be checked to see if it's an UnrealEd URL config key or treated as a raw documentation link as a fallback.
	 *		 See examples of UnrealEd URL config keys in the UnrealEd.URLs section of the BaseEditor.ini config file.
	 */
	UPROPERTY()
	FString From;

	/** To Url */
	UPROPERTY()
	FDocumentationUrl ToUrl;
};
