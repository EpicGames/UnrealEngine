// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DocumentationRedirect.h"
#include "Engine/DeveloperSettings.h"
#include "IDocumentation.h"

#include "DocumentationSettings.generated.h"


USTRUCT()
struct FDocumentationBaseUrl
{
	GENERATED_USTRUCT_BODY()

public:
	UPROPERTY()
		FString Id;

	UPROPERTY()
		FString Url;

	FDocumentationBaseUrl() {};

	FDocumentationBaseUrl(FString const& InId, FString const& InUrl)
		: Id(InId), Url(InUrl)
	{};

	/** Returns true if either the ID or URL is unset. */
	bool IsEmpty() const
	{
		return Id.IsEmpty() || Url.IsEmpty();
	}
};


UCLASS(config=Editor)
class UDocumentationSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	// Array of base URLs for documentation links that are loaded on startup
	UPROPERTY(Config)
	TArray<FDocumentationBaseUrl> DocumentationBaseUrls;

	// Array of static documentation redirects registered on startup
	UPROPERTY(Config)
	TArray<FDocumentationRedirect> DocumentationRedirects;
};