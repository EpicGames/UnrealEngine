// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

#include "DocumentationUrl.generated.h"

USTRUCT()
struct FDocumentationUrl
{
	GENERATED_BODY()

	FDocumentationUrl() = default;
	explicit FDocumentationUrl(const FString& Link) : Link(Link) {}
	FDocumentationUrl(const FString& Link, const FString& BaseUrlId) : Link(Link), BaseUrlId(BaseUrlId) {}

	/** Link for documentation page (page id, full url) */
	UPROPERTY()
	FString Link;

	UPROPERTY()
	FString BaseUrlId;

	bool IsValid() const { return !Link.IsEmpty() || !BaseUrlId.IsEmpty(); }

	FString ToString() const { return !BaseUrlId.IsEmpty() ? (BaseUrlId + ":" + Link) : Link; }
};
