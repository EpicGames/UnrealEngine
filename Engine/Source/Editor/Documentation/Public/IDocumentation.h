// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Modules/ModuleManager.h"
#include "IDocumentationModule.h"

struct FDocumentationRedirect;

template< typename ObjectType > class TAttribute;

/** Invoked when someone clicks on a hyperlink. */
DECLARE_DELEGATE_OneParam( FOnNavigate, const FString& )

class FDocumentationStyle
{
public:
	FDocumentationStyle()
		: ContentStyleName(TEXT("Documentation.Content"))
		, BoldContentStyleName(TEXT("Documentation.BoldContent"))
		, ItalicContentStyleName(TEXT("Documentation.ItalicContent"))
		, NumberedContentStyleName(TEXT("Documentation.NumberedContent"))
		, Header1StyleName(TEXT("Documentation.Header1"))
		, Header2StyleName(TEXT("Documentation.Header2"))
		, Header3StyleName(TEXT("Documentation.Header3"))
		, HyperlinkStyleName(TEXT("Documentation.Hyperlink"))
		, HyperlinkButtonStyleName(TEXT("Documentation.Hyperlink.Button"))
		, HyperlinkTextStyleName(TEXT("Documentation.Hyperlink.Text"))
		, SeparatorStyleName(TEXT("Documentation.Separator"))
	{
	}

	/** Set the content style for this documentation */
	FDocumentationStyle& ContentStyle(const FName& InName) 
	{
		ContentStyleName = InName;
		return *this;
	}

	/** Set the bold content style for this documentation */
	FDocumentationStyle& BoldContentStyle(const FName& InName) 
	{
		BoldContentStyleName = InName;
		return *this;
	}

	/** Set the italic content style for this documentation */
	FDocumentationStyle& ItalicContentStyle(const FName& InName)
	{
		ItalicContentStyleName = InName;
		return *this;
	}

	/** Set the numbered content style for this documentation */
	FDocumentationStyle& NumberedContentStyle(const FName& InName) 
	{
		NumberedContentStyleName = InName;
		return *this;
	}

	/** Set the header 1 style for this documentation */
	FDocumentationStyle& Header1Style(const FName& InName) 
	{
		Header1StyleName = InName;
		return *this;
	}

	/** Set the header 2 style for this documentation */
	FDocumentationStyle& Header2Style(const FName& InName) 
	{
		Header2StyleName = InName;
		return *this;
	}

	/** Set the header 3 style for this documentation */
	FDocumentationStyle& Header3Style(const FName& InName)
	{
		Header3StyleName = InName;
		return *this;
	}

	/** Set the hyperlink button style for this documentation */
	FDocumentationStyle& HyperlinkStyle(const FName& InName) 
	{
		HyperlinkStyleName = InName;
		return *this;
	}

	/** Set the hyperlink button style for this documentation */
	FDocumentationStyle& HyperlinkButtonStyle(const FName& InName) 
	{
		HyperlinkButtonStyleName = InName;
		return *this;
	}

	/** Set the hyperlink text style for this documentation */
	FDocumentationStyle& HyperlinkTextStyle(const FName& InName) 
	{
		HyperlinkTextStyleName = InName;
		return *this;
	}

	/** Set the separator style for this documentation */
	FDocumentationStyle& SeparatorStyle(const FName& InName) 
	{
		SeparatorStyleName = InName;
		return *this;
	}

	/** Content text style */
	FName ContentStyleName;

	/** Bold content text style */
	FName BoldContentStyleName;

	/** Italic content text style */
	FName ItalicContentStyleName;

	/** Numbered content text style */
	FName NumberedContentStyleName;

	/** Header1 text style */
	FName Header1StyleName;

	/** Header2 text style */
	FName Header2StyleName;

	/** Header3 text style */
	FName Header3StyleName;

	/** Hyperlink style */
	FName HyperlinkStyleName;

	/** Hyperlink button style */
	FName HyperlinkButtonStyleName;

	/** Hyperlink text style */
	FName HyperlinkTextStyleName;

	/** Separator style name */
	FName SeparatorStyleName;
};

class FParserConfiguration
{
public:
	static TSharedRef< FParserConfiguration > Create()
	{
		return MakeShareable( new FParserConfiguration() );
	}

public:
	~FParserConfiguration() = default;

	FOnNavigate OnNavigate;

private:
	FParserConfiguration() {}
};


struct FDocumentationSourceInfo
{
public:
	FString Source;
	FString Medium;
	FString Campaign;

	FDocumentationSourceInfo() {};
	FDocumentationSourceInfo(FString const& InCampaign) 
		: Source(TEXT("editor")), Medium(TEXT("docs")), Campaign(InCampaign)
	{};
	FDocumentationSourceInfo(FString const& InSource, FString const& InMedium, FString const& InCampaign)
		: Source(InSource), Medium(InMedium), Campaign(InCampaign)
	{}

	/** Returns true if there is NO valid source info in the struct, false otherwise. */
	bool IsEmpty() const
	{
		return Campaign.IsEmpty() && Source.IsEmpty() && Medium.IsEmpty();
	}
};


class IDocumentation
{
public:

	static inline TSharedRef< IDocumentation > Get()
	{
		IDocumentationModule& Module = FModuleManager::LoadModuleChecked< IDocumentationModule >( "Documentation" );
		return Module.GetDocumentation();
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "Documentation" );
	}

public:

	virtual bool OpenHome(FDocumentationSourceInfo Source = FDocumentationSourceInfo(), const FString& BaseUrlId = FString()) const = 0;

	virtual bool OpenHome(const FCultureRef& Culture, FDocumentationSourceInfo Source = FDocumentationSourceInfo(), const FString & BaseUrlId = FString()) const = 0;

	virtual bool OpenAPIHome(FDocumentationSourceInfo Source = FDocumentationSourceInfo()) const = 0;

	virtual bool Open(const FString& Link, FDocumentationSourceInfo Source = FDocumentationSourceInfo(), const FString& BaseUrlId = FString()) const = 0;

	virtual bool Open(const FString& Link, const FCultureRef& Culture, FDocumentationSourceInfo Source = FDocumentationSourceInfo(), const FString& BaseUrlId = FString()) const = 0;

	virtual TSharedRef< class SWidget > CreateAnchor( const TAttribute<FString>& Link, const FString& PreviewLink = FString(), const FString& PreviewExcerptName = FString(), const TAttribute<FString>& BaseUrlId = FString()) const = 0;

	virtual TSharedRef< class IDocumentationPage > GetPage( const FString& Link, const TSharedPtr< FParserConfiguration >& Config, const FDocumentationStyle& Style = FDocumentationStyle() ) = 0;

	virtual bool PageExists(const FString& Link) const = 0;

	virtual bool PageExists(const FString& Link, const FCultureRef& Culture) const = 0;

	virtual const TArray < FString >& GetSourcePaths() const = 0;

	virtual TSharedRef< class SToolTip > CreateToolTip( const TAttribute<FText>& Text, const TSharedPtr<SWidget>& OverrideContent, const FString& Link, const FString& ExcerptName ) const = 0;

	virtual TSharedRef< class SToolTip > CreateToolTip(const TAttribute<FText>& Text, const TSharedPtr<SWidget>& OverrideContent, const FString& Link, const FString& ExcerptName, const TAttribute<FText>& Shortcut) const = 0;

	virtual TSharedRef< class SToolTip > CreateToolTip(const TAttribute<FText>& Text, const TSharedRef<SWidget>& OverrideContent, const TSharedPtr<class SVerticalBox>& DocVerticalBox, const FString& Link, const FString& ExcerptName) const = 0;

	virtual bool RegisterBaseUrl(const FString& Id, const FString& Url) = 0;

	virtual FString GetBaseUrl(const FString& Id) const = 0;

	/**
	 * Registers a documentation redirect for an owner
	 * @param Owner Name of owner for redirect
	 * @param Redirect Redirect to register
	 * @return Was the redirect successfully registered?
	 */
	virtual bool RegisterRedirect(const FName& Owner, const FDocumentationRedirect& Redirect) = 0;

	/**
	 * Unregisters all redirects owned by a specific owner
	 * @param Owner Name of owner to unregister for
	 */
	virtual void UnregisterRedirects(const FName& Owner) = 0;
};
