// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"

#define UE_API DOCUMENTATION_API

class IDocumentationPage;
class SBox;
class SVerticalBox;
class SWidget;
struct FGeometry;

class SDocumentationToolTip : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS( SDocumentationToolTip )
		: _Text()
		, _Shortcut()
		, _Style( TEXT("Documentation.SDocumentationTooltip") )
		, _SubduedStyle( TEXT("Documentation.SDocumentationTooltipSubdued") )
		, _HyperlinkTextStyle( TEXT("Documentation.SDocumentationTooltipHyperlinkText") )
		, _HyperlinkButtonStyle( TEXT("Documentation.SDocumentationTooltipHyperlinkButton") )
		, _ColorAndOpacity( FLinearColor::Black )
		, _AddDocumentation( true )
		, _DocumentationMargin(0)
		, _OverrideExtendedToolTipContent(nullptr)
		, _OverridePromptContent(nullptr)
		, _AlwaysExpandTooltip(false)
		, _Content()
		{}

		/** The text displayed in this tool tip */
		SLATE_ATTRIBUTE( FText, Text )

		/** The keyboard shortcut to display for the action */
		SLATE_ATTRIBUTE(FText, Shortcut)

		/** The text style to use for this tool tip */
		SLATE_ARGUMENT( FName, Style )
		
		/** The text style to use for subdued footer text in this tool tip */
		SLATE_ARGUMENT( FName, SubduedStyle )
		
		/** The text style to use for hyperlinks in this tool tip */
		SLATE_ARGUMENT( FName, HyperlinkTextStyle )

		/** Hyperlink button style */
		SLATE_ARGUMENT( FName, HyperlinkButtonStyle )

		/** Font color and opacity */
		SLATE_ATTRIBUTE( FSlateColor, ColorAndOpacity )

		/**  */
		SLATE_ARGUMENT( bool, AddDocumentation )

		SLATE_ARGUMENT( FMargin, DocumentationMargin )

		/**  */
		SLATE_ARGUMENT( FString, DocumentationLink )

		/**  */
		SLATE_ARGUMENT( FString, ExcerptName )

		/** Extended ToolTip to use when Ctrl + Alt is pressed instead of loading it from the Documentation */
		SLATE_ARGUMENT(TSharedPtr<SWidget>, OverrideExtendedToolTipContent)

		/** Custom widget to prompt users to hit Ctrl + Alt, if needed to fit with the design of custom content overrides */
		SLATE_ARGUMENT(TSharedPtr<SWidget>, OverridePromptContent)

		/** If true it will keep the tooltip expanded without the need of the Ctrl + Alt to expand it */
		SLATE_ATTRIBUTE(bool, AlwaysExpandTooltip)

		/** Arbitrary content to be displayed in the tool tip; overrides any text that may be set. */
		SLATE_DEFAULT_SLOT( FArguments, Content )

	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	UE_API void Construct( const FArguments& InArgs );

	UE_API void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime );

	UE_API bool IsInteractive() const;

	virtual const FText& GetTextTooltip() const
	{
		return TextContent.Get();
	}

	/**
	 * Adds slots to the provided Vertical Box containing the documentation information.
	 * If you specify not to add it (AddDocumentation = false) you may call this externally to do custom tooltip layout
	 *
	 * @param	VerticalBox	The vertical box to add it to
	 */
	UE_API void AddDocumentation(TSharedPtr< SVerticalBox > VerticalBox);

	UE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

private:

	UE_API EVisibility GetOverriddenFullToolTipVisibility() const;

	UE_API void ConstructSimpleTipContent();

	UE_API void ConstructFullTipContent();

	UE_API FReply ReloadDocumentation();

	UE_API void CreateExcerpt( FString FileSource, FString ExcerptName );

	UE_API EVisibility GetPromptVisibility() const;
	UE_API EVisibility GetFullTipVisibility() const;
	UE_API FOptionalSize GetPromptHeight() const;
	UE_API EVisibility GetControlVisibility() const;
	UE_API EVisibility GetShortcutVisibility() const;

	UE_API bool ShouldAlwaysExpand() const;
private:

	/** Text block widget */
	TAttribute< FText > TextContent;
	TAttribute< FText > Shortcut;
	TSharedPtr< SWidget > OverrideContent;
	TSharedPtr< SWidget > OverrideFullTooltipContent;
	TSharedPtr< SWidget > OverridePromptContent;
	FTextBlockStyle StyleInfo;
	FTextBlockStyle SubduedStyleInfo;
	FTextBlockStyle HyperlinkTextStyleInfo;
	FTextBlockStyle KeybindStyleInfo;
	FButtonStyle HyperlinkButtonStyleInfo;
	TAttribute< FSlateColor > ColorAndOpacity;

	/** The link to the documentation */
	FString DocumentationLink;
	FString ExcerptName;

	/** Content widget */
	TSharedPtr< SBox > WidgetContent;

	TSharedPtr< SBox > SimpleTipContent;
	bool IsDisplayingDocumentationLink;

	TSharedPtr< SBox > FullTipContent;
	TSharedPtr< SBox > PromptContent;
	TSharedPtr< SBox > DocumentationControlBox;
	TSharedPtr< SHorizontalBox > DocumentationControls;

	TSharedPtr< IDocumentationPage > DocumentationPage;
	bool IsShowingFullTip;

	bool bAddDocumentation;
	FMargin DocumentationMargin;

	double TransitionStartTime;
	float TransitionLength;

	bool bFullTipContentIsReady;
	bool bIsPromptVisible;
	TAttribute< bool > AlwaysExpandTooltip;
};

#undef UE_API
