// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Types/ISlateMetaData.h"

class FTutorialMetaData : public FTagMetaData
{
public:
	SLATE_METADATA_TYPE(FTutorialMetaData, FTagMetaData)

	FTutorialMetaData(FName InTag, FString InTabType, FString InFriendlyName = FString())
		: FTagMetaData(InTag)
		, TabTypeToOpen(InTabType)
		, FriendlyName(InFriendlyName)
		{
		}
	
	FTutorialMetaData(FName InTag)
		: FTagMetaData(InTag)
		{
		}
	/* The type of tab we want to open for this widget where relevant. */
	FString TabTypeToOpen;

	/** User friendly display name */
	FString FriendlyName;
};

class FGraphNodeMetaData : public FTutorialMetaData
{
public:
	SLATE_METADATA_TYPE(FGraphNodeMetaData, FTutorialMetaData)

	FGraphNodeMetaData(FName InTag, FString InFriendlyName = FString(), FString InOuterName = FString(), FString InTabType = FString())
		: FTutorialMetaData(InTag, InTabType, InFriendlyName)
		, OuterName(InOuterName)
		{
		}

	/** GUID for the node */
	FGuid GUID;

	/** Name of the outer (which should be the blueprint) */
	FString OuterName;

};
